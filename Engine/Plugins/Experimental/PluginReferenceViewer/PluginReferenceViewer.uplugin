{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Plugin Reference Viewer",
	"Description": "Editor plugin for viewing plugin references.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimental": true,
	"Installed": false,

	"Modules": [
		{
			"Name": "PluginReferenceViewer",
			"Type": "Editor",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "AssetManagerEditor",
			"Enabled": true
		},	
		{
			"Name": "PluginUtils",
			"Enabled": true
		},
		{
			"Name": "PluginBrowser",
			"Enabled": true
		}
	]
}