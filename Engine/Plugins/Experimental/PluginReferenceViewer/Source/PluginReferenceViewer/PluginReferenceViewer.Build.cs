// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using UnrealBuildTool.Rules;

public class PluginReferenceViewer : ModuleRules
{
	public PluginReferenceViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"InputCore",
				"Engine",
				"Slate",
				"SlateCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetManagerEditor",
				"AssetRegistry",
				"EditorFramework",
				"PluginUtils",
				"Projects",
				"ApplicationCore",
				"UnrealEd",
				"GameplayTags",
				"GraphEditor",
				"EditorWidgets",
				"ToolMenus",
				"ToolWidgets",
				"PluginBrowser",
				"ContentBrowserData"
			}
		);
	}
}
