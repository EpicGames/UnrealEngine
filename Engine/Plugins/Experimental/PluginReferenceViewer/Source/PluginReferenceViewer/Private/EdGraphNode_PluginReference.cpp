// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraphNode_PluginReference.h"
#include "EdGraph/EdGraphPin.h"
#include "Interfaces/IPluginManager.h"
#include "PluginReferencePinCategory.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphNode_PluginReference)

#define LOCTEXT_NAMESPACE "PluginReferenceViewer"

//////////////////////////////////////////////////////////////////////////
// UEdGraphNode_Reference

UEdGraphNode_PluginReference::UEdGraphNode_PluginReference(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAllowThumbnail(true)
	, bIsEnginePlugin(false)
	, bIsADuplicate(false)
	, DependencyPin(nullptr)
	, ReferencerPin(nullptr)
{
}

FPluginIdentifier UEdGraphNode_PluginReference::GetIdentifier() const
{
	return PluginIdentifier;
}

void UEdGraphNode_PluginReference::SetupPluginReferenceNode(const FIntPoint& InNodeLoc, const FPluginIdentifier InPluginIdentifier, const TSharedPtr<const IPlugin>& InPlugin, bool bInAllowThumbnail, bool bInIsADuplicate)
{
	NodePosX = InNodeLoc.X;
	NodePosY = InNodeLoc.Y;

	AssetBrush = FSlateIcon(FName("PluginStyle"), "Plugins.TabIcon");

	PluginIdentifier = InPluginIdentifier;

	if (InPlugin.IsValid())
	{
		NodeTitle = FText::FromString(InPlugin->GetFriendlyName());
		bIsEnginePlugin = (InPlugin->GetType() == EPluginType::Engine) ? true : false;
	}
	else
	{
		NodeTitle = FText::Format(LOCTEXT("Title_PluginNotFound", "{0} **Plugin Not Found**"), FText::FromName(InPluginIdentifier.Name));
	}

	bAllowThumbnail = bInAllowThumbnail;
	bIsADuplicate = bInIsADuplicate;

	CachedPlugin = InPlugin;

	AllocateDefaultPins();
}

bool UEdGraphNode_PluginReference::AllowsThumbnail() const
{
	return bAllowThumbnail;
}

TSharedPtr<const IPlugin> UEdGraphNode_PluginReference::GetPlugin() const
{
	return CachedPlugin;
}

UEdGraph_PluginReferenceViewer* UEdGraphNode_PluginReference::GetPluginReferenceViewerGraph() const
{
	return Cast<UEdGraph_PluginReferenceViewer>(GetGraph());
}

void UEdGraphNode_PluginReference::AddReferencer(UEdGraphNode_PluginReference* ReferencerNode)
{
	UEdGraphPin* ReferencerDependencyPin = ReferencerNode->GetDependencyPin();

	if (ensure(ReferencerDependencyPin))
	{
		ReferencerDependencyPin->bHidden = false;
		ReferencerPin->bHidden = false;
		ReferencerPin->MakeLinkTo(ReferencerDependencyPin);
	}
}

void UEdGraphNode_PluginReference::AllocateDefaultPins()
{
	ReferencerPin = CreatePin(EEdGraphPinDirection::EGPD_Input, NAME_None, NAME_None);
	DependencyPin = CreatePin(EEdGraphPinDirection::EGPD_Output, NAME_None, NAME_None);

	ReferencerPin->bHidden = true;
	FName PassiveName = PluginReferencePinUtil::GetName(EPluginReferencePinCategory::LinkEndPassive);
	ReferencerPin->PinType.PinCategory = PassiveName;
	DependencyPin->bHidden = true;
	DependencyPin->PinType.PinCategory = PassiveName;
}

FSlateIcon UEdGraphNode_PluginReference::GetIconAndTint(FLinearColor& OutColor) const
{
	OutColor = FLinearColor::White;
	return AssetBrush;
}

FText UEdGraphNode_PluginReference::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NodeTitle;
}

FLinearColor UEdGraphNode_PluginReference::GetNodeTitleColor() const
{
	if (CachedPlugin.IsValid())
	{
		if (bIsEnginePlugin)
		{
			return FLinearColor(0.55f, 0.55f, 0.55f);
		}
		else
		{
			return FLinearColor(0.0f, 0.42f, 1.0f);
		}
	}
	else
	{
		return FLinearColor(0.55f, 0.55f, 0.55f);
	}
}

FText UEdGraphNode_PluginReference::GetTooltipText() const
{
	if (CachedPlugin.IsValid())
	{
		FString TooltipText = CachedPlugin->GetBaseDir();
		FPaths::MakePathRelativeTo(TooltipText, *FPaths::RootDir());
		return FText::FromString(TooltipText);
	}
	else
	{
		return LOCTEXT("Tooltip_PluginNotFound", "Plugin not found!");
	}
}

UEdGraphPin* UEdGraphNode_PluginReference::GetDependencyPin()
{
	return DependencyPin;
}

UEdGraphPin* UEdGraphNode_PluginReference::GetReferencerPin()
{
	return ReferencerPin;
}

#undef LOCTEXT_NAMESPACE