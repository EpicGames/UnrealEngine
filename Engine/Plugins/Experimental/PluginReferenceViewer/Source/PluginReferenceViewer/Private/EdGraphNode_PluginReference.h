// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph_PluginReferenceViewer.h"
#include "Textures/SlateIcon.h"
#include "EdGraphNode_PluginReference.generated.h"

class IPlugin;
class UEdGraphPin;
class UEdGraph_PluginReferenceViewer;

UCLASS()
class UEdGraphNode_PluginReference : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	FPluginIdentifier GetIdentifier() const;

	void SetupPluginReferenceNode(const FIntPoint& InNodeLoc, const FPluginIdentifier InPluginIdentifier, const TSharedPtr<const IPlugin>& InPlugin, bool bInAllowThumbnail, bool bInIsADuplicate);

	bool AllowsThumbnail() const;
	bool IsADuplicate() const { return bIsADuplicate; }

	TSharedPtr<const IPlugin> GetPlugin() const;

	/** Returns our owning graph */
	UEdGraph_PluginReferenceViewer* GetPluginReferenceViewerGraph() const;
	void AddReferencer(UEdGraphNode_PluginReference* ReferencerNode);

	// UEdGraphNode implementation
	virtual void AllocateDefaultPins() override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual bool ShowPaletteIconOnNode() const override { return true; }
	// End UEdGraphNode implementation
	
	UEdGraphPin* GetDependencyPin();
	UEdGraphPin* GetReferencerPin();

private:
	FPluginIdentifier PluginIdentifier;
	FText NodeTitle;

	bool bAllowThumbnail;
	bool bIsEnginePlugin;
	bool bIsADuplicate;

	TSharedPtr<const IPlugin> CachedPlugin;
	FSlateIcon AssetBrush;

	UEdGraphPin* DependencyPin;
	UEdGraphPin* ReferencerPin;

	friend UEdGraph_PluginReferenceViewer;
};