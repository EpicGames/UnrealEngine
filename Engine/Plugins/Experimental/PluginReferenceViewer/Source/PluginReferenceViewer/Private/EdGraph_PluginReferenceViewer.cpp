// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph_PluginReferenceViewer.h"

#include "Algo/Find.h"
#include "AssetManagerEditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "Async/ParallelFor.h"
#include "EdGraphNode_PluginReference.h"
#include "EdGraph/EdGraphPin.h"
#include "Interfaces/IPluginManager.h"
#include "PluginReferencePinCategory.h"
#include "SPluginReferenceViewer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraph_PluginReferenceViewer)

namespace UE::PluginReferenceViewer::Private
{
	bool ExceedsMaxSearchDepth(int32 CurrentDepth, int32 MaxDepth)
	{
		const bool bIsWithinDepthLimits = MaxDepth > 0 && CurrentDepth < MaxDepth;
		if (!bIsWithinDepthLimits)
		{
			return true;
		}
		return false;
	}

	void FindPluginAssetReferences(const TSharedPtr<IPlugin>& ParentPlugin, const TSharedPtr<IPlugin>& ChildPlugin, TArray<FAssetIdentifier>& OutAssetIdentifiers)
	{
		if (!ParentPlugin.IsValid() || !ChildPlugin.IsValid())
		{
			return;
		}

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		const FString ChildMountedAssetPath = ChildPlugin->GetMountedAssetPath();

		FARFilter Filter;
		Filter.bRecursivePaths = true;
		Filter.PackagePaths.Add(FName(ParentPlugin->GetMountedAssetPath()));

		TArray<FAssetData> AssetsInPlugin;
		AssetRegistry.GetAssets(Filter, AssetsInPlugin);

		// Reusing variables to minimize allocations during large asset list iteration.
		TArray<FAssetDependency> AssetDependencies;
		FString AssetPath;

		for (const FAssetData& AssetData : AssetsInPlugin)
		{
			const FAssetIdentifier AssetIdentifier(AssetData.PackageName);

			AssetDependencies.Empty();
			AssetRegistry.GetDependencies(AssetIdentifier, AssetDependencies);

			for (const FAssetDependency& AssetDependency : AssetDependencies)
			{
				AssetDependency.AssetId.PackageName.ToString(AssetPath);

				if (AssetPath.StartsWith(ChildMountedAssetPath))
				{
					OutAssetIdentifiers.Add(AssetIdentifier);
					break;
				}
			}
		}
	};

	void GetPluginAssets(const TSharedPtr<IPlugin>& Plugin, TArray<FAssetIdentifier>& OutAssetIdentifiers)
	{
		if (!Plugin.IsValid())
		{
			return;
		}

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		FARFilter Filter;
		Filter.bRecursivePaths = true;
		Filter.PackagePaths.Add(FName(Plugin->GetMountedAssetPath()));

		TArray<FAssetData> AssetsInPlugin;
		AssetRegistry.GetAssets(Filter, AssetsInPlugin);

		OutAssetIdentifiers.Reserve(AssetsInPlugin.Num());

		for (int32 Index = 0; Index < AssetsInPlugin.Num(); ++Index)
		{
			OutAssetIdentifiers.Add(FAssetIdentifier(AssetsInPlugin[Index].PackageName));
		}
	};

	uint64 GetSizeOfPlugin(const TSharedPtr<IPlugin> Plugin)
	{
		if (!Plugin.IsValid())
		{
			return 0;
		}

		uint64 Size = 0;
		TArray<FAssetData> PluginAssets;

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		// we want to only include on disk assets otherwise we will be stuck only on the game thread
		AssetRegistry.GetAssetsByPath(FName(Plugin->GetMountedAssetPath()), PluginAssets, true, true);

		for (const FAssetData& PluginAsset : PluginAssets)
		{
			if (TOptional<FAssetPackageData> PackageData = AssetRegistry.GetAssetPackageDataCopy(PluginAsset.PackageName))
			{
				Size += PackageData->DiskSize;
			}
		}

		return Size;
	}

}

FPluginReferenceNodeInfo::FPluginReferenceNodeInfo(const FPluginIdentifier& InIdentifier, bool bInReferencers)
	: Identifier(InIdentifier)
	, bReferencers(bInReferencers)
	, ChildProvisionSize(0)
{

}

bool FPluginReferenceNodeInfo::IsFirstParent(const FPluginIdentifier& InParentId) const
{
	return Parents.IsEmpty() || Parents[0] == InParentId;
}

int32 FPluginReferenceNodeInfo::ProvisionSize(const FPluginIdentifier& InParentId) const
{
	return IsFirstParent(InParentId) ? ChildProvisionSize : 1;
}


UEdGraph_PluginReferenceViewer::UEdGraph_PluginReferenceViewer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bAdvancedInfoLoaded(false)
{

}

void UEdGraph_PluginReferenceViewer::SetGraphRoot(const TArray<FPluginIdentifier>& GraphRootIdentifiers, const FIntPoint& GraphRootOrigin)
{
	CurrentGraphRootIdentifiers = GraphRootIdentifiers;
	CurrentGraphRootOrigin = GraphRootOrigin;
}

const TArray<FPluginIdentifier>& UEdGraph_PluginReferenceViewer::GetCurrentGraphRootIdentifiers() const
{
	return CurrentGraphRootIdentifiers;
}

void UEdGraph_PluginReferenceViewer::FindAssetReferencesAcrossPlugins(const FPluginIdentifier& From, const FPluginIdentifier& To, TArray<FAssetIdentifier>& OutAssetIdentifiers)
{
	const TSharedPtr<IPlugin> FromPlugin = PluginMap.FindRef(From);
	const TSharedPtr<IPlugin> ToPlugin = PluginMap.FindRef(To);

	UE::PluginReferenceViewer::Private::FindPluginAssetReferences(FromPlugin, ToPlugin, OutAssetIdentifiers);
}

void UEdGraph_PluginReferenceViewer::GetPluginAssets(const FPluginIdentifier& Plugin, TArray<FAssetIdentifier>& OutAssetIdentifiers)
{
	const TSharedPtr<IPlugin> ForPlugin = PluginMap.FindRef(Plugin);

	UE::PluginReferenceViewer::Private::GetPluginAssets(ForPlugin, OutAssetIdentifiers);
}

const FPluginStats& UEdGraph_PluginReferenceViewer::GetPluginStats(const FPluginIdentifier& Plugin)
{
	return StatsMap.FindChecked(Plugin);
}

void UEdGraph_PluginReferenceViewer::CachePluginDependencies(const TArray<TSharedRef<IPlugin>>& Plugins)
{
	PluginMap.Empty();

	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		PluginMap.Add(FPluginIdentifier::FromString(Plugin->GetName()), Plugin);
	}

	struct FDependencyNodeBuilder
	{
		TMap<FPluginIdentifier, TUniquePtr<FPluginDependsNode>>& Nodes;

		FDependencyNodeBuilder(TMap<FPluginIdentifier, TUniquePtr<FPluginDependsNode>>& InNodes)
			: Nodes(InNodes)
		{
			Nodes.Empty();
		}

		void ProcessPlugins(const TArray<TSharedRef<IPlugin>>& Plugins)
		{
			for (const TSharedRef<IPlugin>& Plugin : Plugins)
			{
				FPluginIdentifier NewPluginIdentifier = FPluginIdentifier::FromString(Plugin->GetName());
				Nodes.Add(NewPluginIdentifier, MakeUnique<FPluginDependsNode>(NewPluginIdentifier, /* bMissing */ false));
			}

			for (const TSharedRef<IPlugin>& Plugin : Plugins)
			{
				FPluginDependsNode* ParentNode = Nodes.FindChecked(FName(Plugin->GetName())).Get();

				const FPluginDescriptor& PluginDescriptor = Plugin->GetDescriptor();
				for (const FPluginReferenceDescriptor& Item : PluginDescriptor.Plugins)
				{
					FPluginDependsNode* ChildNode = nullptr;
					TUniquePtr<FPluginDependsNode>* FoundNodePtr = Nodes.Find(FPluginIdentifier::FromString(Item.Name));
					if (FoundNodePtr != nullptr)
					{
						ChildNode = FoundNodePtr->Get();
					}
					else
					{
						FPluginIdentifier NewPluginIdentifier = FPluginIdentifier::FromString(Item.Name);
						ChildNode = Nodes.Add(NewPluginIdentifier, MakeUnique<FPluginDependsNode>(NewPluginIdentifier, /* bMissing */ true)).Get();
					}

					ParentNode->Dependencies.Add(ChildNode);
					ChildNode->Referencers.Add(ParentNode);
				}
			}
		}
	};

	// Cache plugin dependencies
	FDependencyNodeBuilder DependencyNodeBuilder(CachedDependsNodes);
	DependencyNodeBuilder.ProcessPlugins(Plugins);
}

UEdGraphNode_PluginReference* UEdGraph_PluginReferenceViewer::ConstructNodes(const TArray<FPluginIdentifier>& GraphRootIdentifiers, const FIntPoint& GraphRootOrigin)
{
	if (!GraphRootIdentifiers.IsEmpty())
	{
		const int32 SettingsReferencerMaxDepth = PluginReferenceViewer.Pin()->GetSearchReferencerDepthCount();
		const int32 SettingsDependencyMaxDepth = PluginReferenceViewer.Pin()->GetSearchDependencyDepthCount();

		TMap<FPluginIdentifier, FPluginReferenceNodeInfo> NewReferenceNodeInfos;
		for (const FPluginIdentifier& RootIdentifier : GraphRootIdentifiers)
		{
			FPluginReferenceNodeInfo& RootNodeInfo = NewReferenceNodeInfos.FindOrAdd(RootIdentifier, FPluginReferenceNodeInfo(RootIdentifier, true));
			RootNodeInfo.Parents.Emplace(NAME_None);
		}

		RecursivelyPopulateNodeInfos(/* bReferencers */ true, GraphRootIdentifiers, NewReferenceNodeInfos, 0, SettingsReferencerMaxDepth);

		TMap<FPluginIdentifier, FPluginReferenceNodeInfo> NewDependencyNodeInfos;
		for (const FPluginIdentifier& RootIdentifier : GraphRootIdentifiers)
		{
			FPluginReferenceNodeInfo& DRootNodeInfo = NewDependencyNodeInfos.FindOrAdd(RootIdentifier, FPluginReferenceNodeInfo(RootIdentifier, false));
			DRootNodeInfo.Parents.Emplace(NAME_None);
		}

		RecursivelyPopulateNodeInfos(/* bReferencers */ false, GraphRootIdentifiers, NewDependencyNodeInfos, 0, SettingsDependencyMaxDepth);

		ReferencerNodeInfos = NewReferenceNodeInfos;
		DependencyNodeInfos = NewDependencyNodeInfos;
	}
	else
	{
		ReferencerNodeInfos.Empty();
		DependencyNodeInfos.Empty();
	}

	return RefilterGraph();
}

void UEdGraph_PluginReferenceViewer::LoadAdvancedPluginInfo()
{
	// don't reload everything if we have already filled the stats map
	if (bAdvancedInfoLoaded)
	{
		return;
	}

	// fill the map before we task work out
	for (const TPair<FPluginIdentifier, TUniquePtr<FPluginDependsNode> >& CachedPair : CachedDependsNodes)
	{
		StatsMap.Add(CachedPair.Key);
	}

	TArray<FPluginIdentifier> MapKeys;
	ReferencerNodeInfos.GenerateKeyArray(MapKeys);

	// GetSizeOfPlugin is an expensive call so using ParallelFors here

	ParallelFor(MapKeys.Num(),
		[this, &MapKeys](int32 Index)
		{
			FPluginStats& Stats = StatsMap.FindChecked(MapKeys[Index]);
			Stats.Size = UE::PluginReferenceViewer::Private::GetSizeOfPlugin(PluginMap.FindRef(MapKeys[Index]));
		});

	MapKeys.Empty();
	DependencyNodeInfos.GenerateKeyArray(MapKeys);

	ParallelFor(MapKeys.Num(),
		[this, &MapKeys](int32 Index)
		{
			FPluginStats& Stats = StatsMap.FindChecked(MapKeys[Index]);
			Stats.Size = UE::PluginReferenceViewer::Private::GetSizeOfPlugin(PluginMap.FindRef(MapKeys[Index]));
		});


	for (TPair<FPluginIdentifier, TUniquePtr<FPluginDependsNode> >& CachedPair : CachedDependsNodes)
	{
		FPluginDependsNode* DependsNode = CachedPair.Value.Get();

		FPluginStats& CurrentPluginStats = StatsMap.FindChecked(CachedPair.Key);

		CurrentPluginStats.Dependencies = DependsNode->Dependencies.Num();
		CurrentPluginStats.Referencers = DependsNode->Referencers.Num();

		CurrentPluginStats.SizeWithDependencies = CurrentPluginStats.Size;

		for (const FPluginDependsNode* Dependency : DependsNode->Dependencies)
		{
			CurrentPluginStats.SizeWithDependencies += StatsMap.FindChecked(Dependency->Identifier).Size;
		}
	}

	bAdvancedInfoLoaded = true;
}

UEdGraphNode_PluginReference* UEdGraph_PluginReferenceViewer::RecursivelyCreateNodes(bool bInReferencers, const FPluginIdentifier& InPluginId, const FIntPoint& InNodeLoc, const FPluginIdentifier& InParentId, UEdGraphNode_PluginReference* InParentNode, TMap<FPluginIdentifier, FPluginReferenceNodeInfo>& InNodeInfos, int32 InCurrentDepth, int32 InMaxDepth, bool bIsRoot)
{
	check(InNodeInfos.Contains(InPluginId));

	const FPluginReferenceNodeInfo& NodeInfo = InNodeInfos[InPluginId];
	int32 NodeProvSize = 1;

	UEdGraphNode_PluginReference* NewNode = nullptr;
	if (bIsRoot)
	{
		NewNode = InParentNode;
		NodeProvSize = NodeInfo.ProvisionSize(FPluginIdentifier(NAME_None));
	}
	else
	{
		const bool bIsCompactMode = PluginReferenceViewer.Pin()->IsCompactModeChecked();
		const bool bIsADuplicate = NodeInfo.Parents.Num() > 1;

		NewNode = CreatePluginReferenceNode();
		NewNode->SetupPluginReferenceNode(InNodeLoc, NodeInfo.Identifier, PluginMap.FindRef(NodeInfo.Identifier), !bIsCompactMode, bIsADuplicate);
		NodeProvSize = NodeInfo.ProvisionSize(InParentId);
	}

	const bool bIsCompactMode = PluginReferenceViewer.Pin()->IsCompactModeChecked();
	const bool bShowDuplicates = PluginReferenceViewer.Pin()->IsShowDuplicatesChecked();

	bool bIsFirstOccurance = bIsRoot || NodeInfo.IsFirstParent(InParentId);
	FIntPoint ChildLoc = InNodeLoc;
	if (!UE::PluginReferenceViewer::Private::ExceedsMaxSearchDepth(InCurrentDepth, InMaxDepth) && bIsFirstOccurance) // Only expand the first parent
	{
		// position the children nodes
		const int32 ColumnWidth = bIsCompactMode ? 500 : 800;
		ChildLoc.X += bInReferencers ? -ColumnWidth : ColumnWidth;

		int32 NodeSizeY = bIsCompactMode ? 100 : 200;
		ChildLoc.Y -= (NodeProvSize - 1) * NodeSizeY * 0.5;

		int32 Breadth = 0;
		int32 ChildIdx = 0;
		for (; ChildIdx < InNodeInfos[InPluginId].Children.Num(); ChildIdx++)
		{
			const TPair<FPluginIdentifier, EPluginReferencePinCategory>& Pair = InNodeInfos[InPluginId].Children[ChildIdx];

			FPluginIdentifier ChildId = Pair.Key;
			int32 ChildProvSize = 0;
			if (InNodeInfos[ChildId].IsFirstParent(InPluginId))
			{
				ChildProvSize = InNodeInfos[ChildId].ProvisionSize(InPluginId);
			}
			else if (bShowDuplicates)
			{
				ChildProvSize = 1;
			}

			// The provision size will always be at least 1 if it should be shown
			if (ChildProvSize > 0)
			{
				ChildLoc.Y += (ChildProvSize - 1) * NodeSizeY * 0.5;

				UEdGraphNode_PluginReference* ChildNode = RecursivelyCreateNodes(bInReferencers, ChildId, ChildLoc, InPluginId, NewNode, InNodeInfos, InCurrentDepth + 1, InMaxDepth);

				if (bInReferencers)
				{
					ChildNode->GetDependencyPin()->PinType.PinCategory = PluginReferencePinUtil::GetName(Pair.Value);
					NewNode->AddReferencer(ChildNode);
				}
				else
				{
					ChildNode->GetReferencerPin()->PinType.PinCategory = PluginReferencePinUtil::GetName(Pair.Value);
					ChildNode->AddReferencer(NewNode);
				}

				ChildLoc.Y += NodeSizeY * (ChildProvSize + 1) * 0.5;
				Breadth++;
			}
		}
	}

	return NewNode;
}

void UEdGraph_PluginReferenceViewer::GetSortedLinks(const TArray<FPluginIdentifier>& Identifiers, bool bReferencers, TMap<FPluginIdentifier, EPluginReferencePinCategory>& OutLinks) const
{
	const bool bShowEnginePlugins = PluginReferenceViewer.Pin()->IsShowEnginePluginsChecked();

	TArray<FPluginDependency> LinksToAsset;

	for (const FPluginIdentifier& PluginIdentifier : Identifiers)
	{
		LinksToAsset.Reset();
		if (bReferencers)
		{
			GetPluginReferencers(PluginIdentifier, LinksToAsset);
		}
		else
		{
			GetPluginDependencies(PluginIdentifier, LinksToAsset);
		}

		for (const FPluginDependency& LinkToAsset : LinksToAsset)
		{
			EPluginReferencePinCategory& Category = OutLinks.FindOrAdd(LinkToAsset.Identifier, EPluginReferencePinCategory::LinkEndActive);
			Category |= LinkToAsset.bIsEnabled ? EPluginReferencePinCategory::LinkTypeEnabled : EPluginReferencePinCategory::LinkTypeNone;
			Category |= LinkToAsset.bIsOptional ? EPluginReferencePinCategory::LinkTypeOptional : EPluginReferencePinCategory::LinkTypeNone;
		}
	}
}

void UEdGraph_PluginReferenceViewer::RecursivelyPopulateNodeInfos(bool bInReferencers, const TArray<FPluginIdentifier>& Identifiers, TMap<FPluginIdentifier, FPluginReferenceNodeInfo>& InNodeInfos, int32 InCurrentDepth, int32 InMaxDepth)
{
	check(Identifiers.Num() > 0);
	int32 ProvisionSize = 0;
	const FPluginIdentifier PluginId = Identifiers[0];
	if (!UE::PluginReferenceViewer::Private::ExceedsMaxSearchDepth(InCurrentDepth, InMaxDepth))
	{
		TMap<FPluginIdentifier, EPluginReferencePinCategory> ReferenceLinks;
		GetSortedLinks(Identifiers, bInReferencers, ReferenceLinks);

		InNodeInfos[PluginId].Children.Reserve(ReferenceLinks.Num());
		for (const TPair<FPluginIdentifier, EPluginReferencePinCategory>& Pair : ReferenceLinks)
		{
			FPluginIdentifier ChildId = Pair.Key;
			if (!InNodeInfos.Contains(ChildId))
			{
				FPluginReferenceNodeInfo& NewNodeInfo = InNodeInfos.FindOrAdd(ChildId, FPluginReferenceNodeInfo(ChildId, bInReferencers));
				InNodeInfos[ChildId].Parents.Emplace(PluginId);
				InNodeInfos[PluginId].Children.Emplace(Pair);

				RecursivelyPopulateNodeInfos(bInReferencers, { ChildId }, InNodeInfos, InCurrentDepth + 1, InMaxDepth);
				ProvisionSize += InNodeInfos[ChildId].ProvisionSize(PluginId);
			}
			else if (!InNodeInfos[ChildId].Parents.Contains(PluginId))
			{
				InNodeInfos[ChildId].Parents.Emplace(PluginId);
				InNodeInfos[PluginId].Children.Emplace(Pair);
				ProvisionSize += 1;
			}
		}
	}

	InNodeInfos[PluginId].ChildProvisionSize = ProvisionSize > 0 ? ProvisionSize : 1;
}

UEdGraphNode_PluginReference* UEdGraph_PluginReferenceViewer::RebuildGraph()
{
	RemoveAllNodes();

	UEdGraphNode_PluginReference* NewRootNode = ConstructNodes(CurrentGraphRootIdentifiers, CurrentGraphRootOrigin);

	if (bAdvancedInfoLoaded)
	{
		// Load advanced plugin checks this bool so multiple load requests from the UI don't cause needless reloads, so set false
		bAdvancedInfoLoaded = false;

		StatsMap.Empty();
		LoadAdvancedPluginInfo();
	}

	return NewRootNode;
}

UEdGraphNode_PluginReference* UEdGraph_PluginReferenceViewer::RefilterGraph()
{
	RemoveAllNodes();

	UEdGraphNode_PluginReference* RootNode = nullptr;

	if (CurrentGraphRootIdentifiers.Num() > 0 && (!ReferencerNodeInfos.IsEmpty() || !DependencyNodeInfos.IsEmpty()))
	{
		const TSharedPtr<const SPluginReferenceViewer> LocalPluginReferenceViewer = PluginReferenceViewer.Pin();

		const bool bIsCompactMode = LocalPluginReferenceViewer->IsCompactModeChecked();
		const int32 SettingsReferencerMaxDepth = LocalPluginReferenceViewer->GetSearchReferencerDepthCount();
		const int32 SettingsDependencyMaxDepth = LocalPluginReferenceViewer->GetSearchDependencyDepthCount();

		FPluginIdentifier FirstGraphRootIdentifier = CurrentGraphRootIdentifiers[0];

		const FPluginReferenceNodeInfo& NodeInfo = ReferencerNodeInfos[FirstGraphRootIdentifier];
		RootNode = CreatePluginReferenceNode();
		RootNode->SetupPluginReferenceNode(CurrentGraphRootOrigin, NodeInfo.Identifier, PluginMap.FindRef(NodeInfo.Identifier), !bIsCompactMode, false);

		// Show referencers
		RecursivelyFilterNodeInfos(FirstGraphRootIdentifier, ReferencerNodeInfos, 0, SettingsReferencerMaxDepth);
		RecursivelyCreateNodes(true, FirstGraphRootIdentifier, CurrentGraphRootOrigin, FirstGraphRootIdentifier, RootNode, ReferencerNodeInfos, 0, SettingsReferencerMaxDepth, /*bIsRoot*/ true);

		// Show dependencies
		RecursivelyFilterNodeInfos(FirstGraphRootIdentifier, DependencyNodeInfos, 0, SettingsDependencyMaxDepth);
		RecursivelyCreateNodes(false, FirstGraphRootIdentifier, CurrentGraphRootOrigin, FirstGraphRootIdentifier, RootNode, DependencyNodeInfos, 0, SettingsDependencyMaxDepth, /*bIsRoot*/ true);
	}

	return RootNode;
}

void UEdGraph_PluginReferenceViewer::RecursivelyFilterNodeInfos(const FPluginIdentifier& InPluginId, TMap<FPluginIdentifier, FPluginReferenceNodeInfo>& InNodeInfos, int32 InCurrentDepth, int32 InMaxDepth)
{
	const bool bShowDuplicates = PluginReferenceViewer.Pin()->IsShowDuplicatesChecked();

	// Filters and Re-provisions the NodeInfo counts 
	int32 NewProvisionSize = 0;

	if (!UE::PluginReferenceViewer::Private::ExceedsMaxSearchDepth(InCurrentDepth, InMaxDepth))
	{
		for (const TPair<FPluginIdentifier, EPluginReferencePinCategory>& Pair : InNodeInfos[InPluginId].Children)
		{
			FPluginIdentifier ChildId = Pair.Key;

			int32 ChildProvSize = 0;
			if (InNodeInfos[ChildId].IsFirstParent(InPluginId))
			{
				RecursivelyFilterNodeInfos(ChildId, InNodeInfos, InCurrentDepth + 1, InMaxDepth);
				ChildProvSize = InNodeInfos[ChildId].ProvisionSize(InPluginId);
			}
			else if (bShowDuplicates)
			{
				ChildProvSize = 1;
			}

			if (ChildProvSize > 0)
			{
				NewProvisionSize += ChildProvSize;
			}
		}
	}

	InNodeInfos[InPluginId].ChildProvisionSize = NewProvisionSize > 0 ? NewProvisionSize : 1;
}

UEdGraphNode_PluginReference* UEdGraph_PluginReferenceViewer::CreatePluginReferenceNode()
{
	const bool bSelectNewNode = false;
	return Cast<UEdGraphNode_PluginReference>(CreateNode(UEdGraphNode_PluginReference::StaticClass(), bSelectNewNode));
}

void UEdGraph_PluginReferenceViewer::RemoveAllNodes()
{
	TArray<UEdGraphNode*> NodesToRemove = Nodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		RemoveNode(NodesToRemove[NodeIndex]);
	}
}

bool UEdGraph_PluginReferenceViewer::IsDependencyNode(const FPluginIdentifier& PluginIdentifier) const
{
	return DependencyNodeInfos.Contains(PluginIdentifier);
}

bool UEdGraph_PluginReferenceViewer::IsReferencerNode(const FPluginIdentifier& PluginIdentifier) const
{
	return ReferencerNodeInfos.Contains(PluginIdentifier);
}

void UEdGraph_PluginReferenceViewer::SetPluginReferenceViewer(TSharedPtr<SPluginReferenceViewer> InViewer)
{
	PluginReferenceViewer = InViewer;
}

void UEdGraph_PluginReferenceViewer::GetPluginDependencies(const FPluginIdentifier& PluginIdentifier, TArray<FPluginDependency>& OutDependencies) const
{
	const bool bShowEnginePlugins = PluginReferenceViewer.Pin()->IsShowEnginePluginsChecked();
	const bool bShowOptionalPlugins = PluginReferenceViewer.Pin()->IsShowOptionalPluginsChecked();

	const FPluginDependsNode* PluginNode = CachedDependsNodes.FindChecked(PluginIdentifier).Get();
	const TSharedPtr<IPlugin> Plugin = PluginMap.FindRef(PluginNode->Identifier);
	if (!Plugin.IsValid()) // Missing optional plugin
	{
		return;
	}

	for (int32 i = 0; i < PluginNode->Dependencies.Num(); ++i)
	{
		const FPluginDependsNode* ChildNode = PluginNode->Dependencies[i];

		// Plugin dependencies may not be discovered
		if (const TSharedPtr<IPlugin> ChildPlugin = PluginMap.FindRef(ChildNode->Identifier))
		{
			if (ChildPlugin->GetType() == EPluginType::Engine && !bShowEnginePlugins)
			{
				continue;
			}
		}

		const FPluginReferenceDescriptor& ReferenceDescriptor = Plugin->GetDescriptor().Plugins[i];
		check(ReferenceDescriptor.Name == ChildNode->Identifier.Name.ToString());

		if (ReferenceDescriptor.bOptional && !bShowOptionalPlugins)
		{
			continue;
		}

		OutDependencies.Add(FPluginDependency(ChildNode->Identifier, ReferenceDescriptor.bEnabled, ReferenceDescriptor.bOptional));
	}
}

void UEdGraph_PluginReferenceViewer::GetPluginReferencers(const FPluginIdentifier& PluginIdentifier, TArray<FPluginDependency>& OutReferencers) const
{
	const FPluginDependsNode* PluginNode = CachedDependsNodes.FindChecked(PluginIdentifier).Get();
	for (const FPluginDependsNode* Referencer : PluginNode->Referencers)
	{
		const TSharedPtr<IPlugin> ReferencerPlugin = PluginMap.FindRef(Referencer->Identifier);
		if (!ReferencerPlugin.IsValid()) // Should not happen as we only make plugin A -> plugin B references when looking at plugin A in the plugin manager 
		{
			continue;
		}

		FPluginDependsNode* const* FoundDependency = Algo::Find(Referencer->Dependencies, PluginNode);
		checkf(FoundDependency, TEXT("Referencer mapping from %s to %s is not symmetric"), *Referencer->Identifier.ToString(), *PluginIdentifier.ToString());
		
		// Indices are assumed to match from iteration order in CachePluginDependencies
		int32 Index = FoundDependency - Referencer->Dependencies.GetData();

		const FPluginReferenceDescriptor& ReferenceDescriptor = ReferencerPlugin->GetDescriptor().Plugins[Index];
		check(ReferenceDescriptor.Name == PluginNode->Identifier.Name.ToString());

		OutReferencers.Add(FPluginDependency(Referencer->Identifier, ReferenceDescriptor.bEnabled, ReferenceDescriptor.bOptional));
	}
}

