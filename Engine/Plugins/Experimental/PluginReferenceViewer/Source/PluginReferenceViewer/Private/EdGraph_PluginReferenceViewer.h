// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "PluginReferencePinCategory.h"

#include "EdGraph_PluginReferenceViewer.generated.h"

class IPlugin;
class UEdGraphNode_PluginReference;
class SPluginReferenceViewer;
struct FAssetIdentifier;

struct FPluginIdentifier
{
	FName Name;

	FPluginIdentifier()
		: Name(NAME_None)
	{
	}

	FPluginIdentifier(FName InName)
		: Name(InName)
	{
	}

	friend inline bool operator==(const FPluginIdentifier& A, const FPluginIdentifier& B)
	{
		return A.Name == B.Name;
	}

	friend inline uint32 GetTypeHash(const FPluginIdentifier& Key)
	{
		return GetTypeHash(Key.Name);
	}

	bool IsValid() const
	{
		return Name != NAME_None;
	}

	FString ToString() const
	{
		return Name.ToString();
	}

	static FPluginIdentifier FromString(const FString& String)
	{
		return FPluginIdentifier(FName(String));
	}
};


struct FAssetStats
{
	FAssetStats() : References(0)
	{
	}

	uint32 References;
};

struct FPluginStats
{
	uint64 Size;
	uint64 SizeWithDependencies;

	uint32 Dependencies;
	uint32 Referencers;
};


struct FPluginDependsNode
{
	explicit FPluginDependsNode(const FPluginIdentifier& InIdentifier, bool InMissing)
		: Identifier(InIdentifier)
		, bMissing(InMissing)
	{
	}

	FPluginIdentifier Identifier;
	TArray<FPluginDependsNode*> Dependencies; // Raw pointer to another node in CachedDependsNodes, same lifetime
	TArray<FPluginDependsNode*> Referencers; // Raw pointer to another node in CachedDependsNodes, same lifetime
	bool bMissing = false;
};

struct FPluginReferenceNodeInfo
{
	FPluginIdentifier Identifier;

	TArray<TPair<FPluginIdentifier, EPluginReferencePinCategory>> Children;

	TArray<FPluginIdentifier> Parents;

	bool bReferencers;

	FPluginReferenceNodeInfo(const FPluginIdentifier& InIdentifier, bool bInReferencers);

	bool IsFirstParent(const FPluginIdentifier& InParentId) const;

	// The Provision Size, or vertical spacing required for layout, for a given parent.  
	int32 ProvisionSize(const FPluginIdentifier& InParentId) const;

	// how many nodes worth of children require vertical spacing 
	int32 ChildProvisionSize;
};

UCLASS()
class UEdGraph_PluginReferenceViewer : public UEdGraph
{
	GENERATED_UCLASS_BODY()

public:
	void CachePluginDependencies(const TArray<TSharedRef<IPlugin>>& Plugins);

	/** Set reference viewer to focus on these assets */
	void SetGraphRoot(const TArray<FPluginIdentifier>& GraphRootIdentifiers, const FIntPoint& GraphRootOrigin = FIntPoint(ForceInitToZero));

	const TArray<FPluginIdentifier>& GetCurrentGraphRootIdentifiers() const;

	/**
	 * Finds all plugin assets that have dependencies on another plugins assets
	 *
	 * @param From the plugin whose assets we want to find references from
	 * @param To the plugin whose assets we want to find dependencies to
	 */
	void FindAssetReferencesAcrossPlugins(const FPluginIdentifier& From, const FPluginIdentifier& To, TArray<FAssetIdentifier>& OutAssetIdentifiers);

	/** Finds all plugin assets */
	void GetPluginAssets(const FPluginIdentifier& Plugin, TArray<FAssetIdentifier>& OutAssetIdentifiers);

	const FPluginStats& GetPluginStats(const FPluginIdentifier& Plugin);

	void LoadAdvancedPluginInfo();

	UEdGraphNode_PluginReference* ConstructNodes(const TArray<FPluginIdentifier>& GraphRootIdentifiers, const FIntPoint& GraphRootOrigin);

	UEdGraphNode_PluginReference* RecursivelyCreateNodes(bool bInReferencers, const FPluginIdentifier& InPluginId, const FIntPoint& InNodeLoc, const FPluginIdentifier& InParentId, UEdGraphNode_PluginReference* InParentNode, TMap<FPluginIdentifier, FPluginReferenceNodeInfo>& InNodeInfos, int32 InCurrentDepth, int32 InMaxDepth, bool bIsRoot = false);

	void RecursivelyPopulateNodeInfos(bool bInReferencers, const TArray<FPluginIdentifier>& Identifiers, TMap<FPluginIdentifier, FPluginReferenceNodeInfo>& InNodeInfos, int32 InCurrentDepth, int32 InMaxDepth);

	void RecursivelyFilterNodeInfos(const FPluginIdentifier& InPluginId, TMap<FPluginIdentifier, FPluginReferenceNodeInfo>& InNodeInfos, int32 InCurrentDepth, int32 InMaxDepth);

	void GetSortedLinks(const TArray<FPluginIdentifier>& Identifiers, bool bReferencers, TMap<FPluginIdentifier, EPluginReferencePinCategory>& OutLinks) const;

	/** Force the graph to rebuild */
	UEdGraphNode_PluginReference* RebuildGraph();

	UEdGraphNode_PluginReference* CreatePluginReferenceNode();
	void RemoveAllNodes();

	/** Returns true if the plugin is a dependency of the graph root plugin. */
	bool IsDependencyNode(const FPluginIdentifier& PluginIdentifier) const;

	/** Returns true if the plugin is a referencer of the graph root plugin. */
	bool IsReferencerNode(const FPluginIdentifier& PluginIdentifier) const;

private:
	struct FPluginDependency
	{
		FPluginIdentifier Identifier;
		bool bIsEnabled;
		bool bIsOptional;

		FPluginDependency(FPluginIdentifier InIdentifier, bool bInIsEnabled, bool bInIsOptional)
			: Identifier(InIdentifier)
			, bIsEnabled(bInIsEnabled)
			, bIsOptional(bInIsOptional)
		{
		}
	};

	void SetPluginReferenceViewer(TSharedPtr<SPluginReferenceViewer> InViewer);

	void GetPluginDependencies(const FPluginIdentifier& PluginIdentifier, TArray<FPluginDependency>& OutDependencies) const;
	void GetPluginReferencers(const FPluginIdentifier& PluginIdentifier, TArray<FPluginDependency>& OutReferencers) const;

	UEdGraphNode_PluginReference* RefilterGraph();

private:
	/** Editor for this pool */
	TWeakPtr<SPluginReferenceViewer> PluginReferenceViewer;

	TArray<FPluginIdentifier> CurrentGraphRootIdentifiers;
	FIntPoint CurrentGraphRootOrigin;

	// Mapping of plugins which were seen when traversing the dependency graph of active plugins, whether or not they exist in the plugin manager 
	TMap<FPluginIdentifier, TUniquePtr<FPluginDependsNode>> CachedDependsNodes;
	// Mapping of plugins which actually exist - may have fewer keys than CachedDependsNodes
	TMap<FPluginIdentifier, const TSharedPtr<IPlugin>> PluginMap;
	TMap<FPluginIdentifier, FPluginStats> StatsMap;

	// Current set of referencers of the graph root plugin(s) to be displayed, expanded to configured depth
	TMap<FPluginIdentifier, FPluginReferenceNodeInfo> ReferencerNodeInfos;
	// Current set of dependencies of the graph root plugin(s) to be displayed, expanded to configured depth 
	TMap<FPluginIdentifier, FPluginReferenceNodeInfo> DependencyNodeInfos;

	friend SPluginReferenceViewer;

	bool bAdvancedInfoLoaded = false;
};