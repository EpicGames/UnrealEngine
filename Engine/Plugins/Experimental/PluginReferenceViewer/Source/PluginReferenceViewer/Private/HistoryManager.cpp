// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryManager.h"
#include "EdGraph_PluginReferenceViewer.h"

FPluginReferenceViewerHistoryManager::FPluginReferenceViewerHistoryManager()
{
	CurrentHistoryIndex = 0;
	MaxHistoryEntries = 300;
}

void FPluginReferenceViewerHistoryManager::SetOnApplyHistoryData(const FOnApplyHistoryData& InOnApplyHistoryData)
{
	OnApplyHistoryData = InOnApplyHistoryData;
}

void FPluginReferenceViewerHistoryManager::SetOnUpdateHistoryData(const FOnUpdateHistoryData& InOnUpdateHistoryData)
{
	OnUpdateHistoryData = InOnUpdateHistoryData;
}

bool FPluginReferenceViewerHistoryManager::GoBack()
{
	if ( CanGoBack() )
	{
		// Update the current history data
		UpdateCurrentHistoryData();

		// if its possible to go back, decrement the index we are at
		--CurrentHistoryIndex;

		// Update the owner
		ApplyCurrentHistoryData();

		return true;
	}

	return false;
}

bool FPluginReferenceViewerHistoryManager::GoForward()
{
	if ( CanGoForward() )
	{
		// Update the current history data
		UpdateCurrentHistoryData();

		// if its possible to go forward, increment the index we are at
		++CurrentHistoryIndex;

		// Update the owner
		ApplyCurrentHistoryData();

		return true;
	}

	return false;
}

void FPluginReferenceViewerHistoryManager::AddHistoryData()
{
	if (HistoryData.Num() == 0)
	{
		// History added to the beginning
		HistoryData.Add(FPluginReferenceViewerHistoryData());
		CurrentHistoryIndex = 0;
	}
	else if (CurrentHistoryIndex == HistoryData.Num() - 1)
	{
		// History added to the end
		if (HistoryData.Num() == MaxHistoryEntries)
		{
			// If max history entries has been reached
			// remove the oldest history
			HistoryData.RemoveAt(0);
		}
		HistoryData.Add(FPluginReferenceViewerHistoryData());
		// Current history index is the last index in the list
		CurrentHistoryIndex = HistoryData.Num() - 1;
	}
	else
	{
		// History added to the middle
		// clear out all history after the current history index.
		HistoryData.RemoveAt(CurrentHistoryIndex + 1, HistoryData.Num() - (CurrentHistoryIndex + 1));
		HistoryData.Add(FPluginReferenceViewerHistoryData());
		// Current history index is the last index in the list
		CurrentHistoryIndex = HistoryData.Num() - 1;
	}

	// Update the current history data
	UpdateCurrentHistoryData();
}

void FPluginReferenceViewerHistoryManager::UpdateHistoryData()
{
	// Update the current history data
	UpdateCurrentHistoryData();
}

bool FPluginReferenceViewerHistoryManager::CanGoForward() const
{
	// User can go forward if there are items in the history data list, 
	// and the current history index isn't the last index in the list
	return HistoryData.Num() > 0 && CurrentHistoryIndex < HistoryData.Num() - 1;
}

bool FPluginReferenceViewerHistoryManager::CanGoBack() const
{
	// User can go back if there are items in the history data list,
	// and the current history index isn't the first index in the list
	return HistoryData.Num() > 0 && CurrentHistoryIndex > 0;
}

FText FPluginReferenceViewerHistoryManager::GetBackDesc() const
{
	if ( CanGoBack() )
	{
		return HistoryData[CurrentHistoryIndex - 1].HistoryDesc;
	}
	return FText::GetEmpty();
}

FText FPluginReferenceViewerHistoryManager::GetForwardDesc() const
{
	if ( CanGoForward() )
	{
		return HistoryData[CurrentHistoryIndex + 1].HistoryDesc;
	}
	return FText::GetEmpty();
}

void FPluginReferenceViewerHistoryManager::ApplyCurrentHistoryData()
{
	if ( CurrentHistoryIndex >= 0 && CurrentHistoryIndex < HistoryData.Num())
	{
		OnApplyHistoryData.ExecuteIfBound( HistoryData[CurrentHistoryIndex] );
	}
}

void FPluginReferenceViewerHistoryManager::UpdateCurrentHistoryData()
{
	if ( CurrentHistoryIndex >= 0 && CurrentHistoryIndex < HistoryData.Num())
	{
		OnUpdateHistoryData.ExecuteIfBound( HistoryData[CurrentHistoryIndex] );
	}
}

void FPluginReferenceViewerHistoryManager::ExecuteJumpToHistory(int32 HistoryIndex)
{
	if (HistoryIndex >= 0 && HistoryIndex < HistoryData.Num())
	{
		// if the history index is valid, set the current history index to the history index requested by the user
		CurrentHistoryIndex = HistoryIndex;

		// Update the owner
		ApplyCurrentHistoryData();
	}
}