// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

struct FPluginIdentifier;

class FMenuBuilder;

/** The history data object, storing all important history data */
struct FPluginReferenceViewerHistoryData
{
	/** History Description */
	FText HistoryDesc;

	/** The list of package names to serve as the root */
	TArray<FPluginIdentifier> Identifiers;
};

/** The delegate for when history data should be applied */
DECLARE_DELEGATE_OneParam(FOnApplyHistoryData, const FPluginReferenceViewerHistoryData& /*Data*/);

/** The delegate for when history data should be updated */
DECLARE_DELEGATE_OneParam(FOnUpdateHistoryData, FPluginReferenceViewerHistoryData& /*Data*/);

/** The class responsible for managing all content browser history */
class FPluginReferenceViewerHistoryManager
{
public:
	/** Constructor */
	FPluginReferenceViewerHistoryManager();

	/** Set the delegate for applying history data */
	void SetOnApplyHistoryData(const FOnApplyHistoryData& InOnApplyHistoryData);

	/** Set the delegate for updating history data */
	void SetOnUpdateHistoryData(const FOnUpdateHistoryData& InOnUpdateHistoryData);

	/** Goes back one history snapshot and returns the history data at that snapshot */
	bool GoBack();

	/** Goes forward one history snapshot and returns the history data at that snapshot */
	bool GoForward();

	/** Stores new history data.  Called when creating a history snapshot */
	void AddHistoryData();

	/** Triggers an update for the current history data. This is typically done right before changing the history. */
	void UpdateHistoryData();

	/** Determines if a user can go forward in history */
	bool CanGoForward() const;

	/** Determines if a user can go back in history */
	bool CanGoBack() const;

	/** Gets the description of the previous history entry */
	FText GetBackDesc() const;

	/** Gets the description of the next history entry */
	FText GetForwardDesc() const;

private:
	/** Notifies the owner to update to the state described by the current history data */
	void ApplyCurrentHistoryData();

	/** Notifies the owner to update the current history data */
	void UpdateCurrentHistoryData();

	/** Handler for when a history item is chosen in the GetAvailableHistroyMenuItems list */
	void ExecuteJumpToHistory(int32 HistoryIndex);

private:
	/** The delegate for when history data should be applied */
	FOnApplyHistoryData OnApplyHistoryData;

	/** The delegate for when history data should be updated */
	FOnUpdateHistoryData OnUpdateHistoryData;

	/** A list of history snapshots */
	TArray<FPluginReferenceViewerHistoryData> HistoryData;

	/** The current history index the user is at (changes when the user goes back,forward, or history snapshots are taken) */
	int32 CurrentHistoryIndex;

	/** Max number of history items that can be stored.  Once the max is reached, the oldest history item is removed */
	int32 MaxHistoryEntries;
};
