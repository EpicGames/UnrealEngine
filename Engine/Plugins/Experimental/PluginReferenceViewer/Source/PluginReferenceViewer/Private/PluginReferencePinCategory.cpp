// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginReferencePinCategory.h"

namespace PluginDependencyPinCategory
{
	FName NamePassive(TEXT("Passive"));
	FName NameHard(TEXT("Hard"));
	FName NameHardOptional(TEXT("HardOptional"));
	FName NameSoft(TEXT("Soft"));
	FName NameSoftOptional(TEXT("SoftOptional"));
	const FLinearColor ColorPassive = FLinearColor(128, 128, 128);
	const FLinearColor ColorHard = FLinearColor(FColor(236, 252, 227)); // RiceFlower
	const FLinearColor ColorHardOptional = FLinearColor(FColor(118, 126, 114));
	const FLinearColor ColorSoft = FLinearColor(FColor(145, 66, 117)); // CannonPink
	const FLinearColor ColorSoftOptional = FLinearColor(FColor(73, 33, 58));
}

namespace PluginReferencePinUtil
{
	EPluginReferencePinCategory ParseDependencyPinCategory(FName PinCategory)
	{
		if (PinCategory == PluginDependencyPinCategory::NameHardOptional)
		{
			return EPluginReferencePinCategory::LinkEndActive | EPluginReferencePinCategory::LinkTypeEnabled | EPluginReferencePinCategory::LinkTypeOptional;
		}
		else if (PinCategory == PluginDependencyPinCategory::NameHard)
		{
			return EPluginReferencePinCategory::LinkEndActive | EPluginReferencePinCategory::LinkTypeEnabled;
		}
		else if (PinCategory == PluginDependencyPinCategory::NameSoftOptional)
		{
			return EPluginReferencePinCategory::LinkEndActive | EPluginReferencePinCategory::LinkTypeOptional;
		}
		else if (PinCategory == PluginDependencyPinCategory::NameSoft)
		{
			return EPluginReferencePinCategory::LinkEndActive;
		}
		else
		{
			return EPluginReferencePinCategory::LinkEndPassive;
		}
	}

	FName GetName(EPluginReferencePinCategory Category)
	{
		if ((Category & EPluginReferencePinCategory::LinkEndMask) == EPluginReferencePinCategory::LinkEndPassive)
		{
			return PluginDependencyPinCategory::NamePassive;
		}
		else
		{
			switch (Category & EPluginReferencePinCategory::LinkTypeMask)
			{
			case EPluginReferencePinCategory::LinkTypeEnabled | EPluginReferencePinCategory::LinkTypeOptional:
				return PluginDependencyPinCategory::NameHardOptional;
			case EPluginReferencePinCategory::LinkTypeEnabled:
				return PluginDependencyPinCategory::NameHard;
			case EPluginReferencePinCategory::LinkTypeOptional:
				return PluginDependencyPinCategory::NameSoftOptional;
			default:
				return PluginDependencyPinCategory::NameSoft;
			}
		}
	}


	FLinearColor GetColor(EPluginReferencePinCategory Category)
	{
		if ((Category & EPluginReferencePinCategory::LinkEndMask) == EPluginReferencePinCategory::LinkEndPassive)
		{
			return PluginDependencyPinCategory::ColorPassive;
		}
		else
		{
			switch (Category & EPluginReferencePinCategory::LinkTypeMask)
			{
			case EPluginReferencePinCategory::LinkTypeEnabled | EPluginReferencePinCategory::LinkTypeOptional:
				return PluginDependencyPinCategory::ColorHardOptional;
			case EPluginReferencePinCategory::LinkTypeEnabled:
				return PluginDependencyPinCategory::ColorHard;
			case EPluginReferencePinCategory::LinkTypeOptional:
				return PluginDependencyPinCategory::ColorSoftOptional;
			default:
				return PluginDependencyPinCategory::ColorSoft;
			}
		}
	}
}