// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class EPluginReferencePinCategory
{
	LinkEndPassive = 0,
	LinkEndActive = 1,
	LinkEndMask = LinkEndActive,

	LinkTypeNone = 0,
	LinkTypeOptional = 2,
	LinkTypeEnabled = 4,
	LinkTypeMask = LinkTypeEnabled | LinkTypeOptional,
};
ENUM_CLASS_FLAGS(EPluginReferencePinCategory);

namespace PluginReferencePinUtil
{
	EPluginReferencePinCategory ParseDependencyPinCategory(FName PinCategory);
	FName GetName(EPluginReferencePinCategory Category);
	FLinearColor GetColor(EPluginReferencePinCategory Category);
}