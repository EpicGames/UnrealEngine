// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginReferenceViewerCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "PluginReferenceViewerCommands"

FPluginReferenceViewerCommands::FPluginReferenceViewerCommands() : TCommands<FPluginReferenceViewerCommands>(
	"PluginReferenceViewerCommands",
	NSLOCTEXT("Contexts", "PluginReferenceViewerCommands", "Plugin Reference Viewer"),
	NAME_None,
	FAppStyle::GetAppStyleSetName())
{
}

void FPluginReferenceViewerCommands::RegisterCommands()
{	
	UI_COMMAND(CompactMode, "Compact Mode", "Toggles Compact View", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::V));
	UI_COMMAND(ShowDuplicates, "Show Duplicates References", "Toggles visibility of duplicates references", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::U));
	UI_COMMAND(ShowEnginePlugins, "Show Engine Plugins", "Toggles visibility of Engine Plugins", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowOptionalPlugins, "Show Optional Plugins", "Toggles visibility of Optional Plugin dependencies", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(OpenPluginProperties, "Open Plugin Properties", "Opens the Plugin Properties Editor Window", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenAssetDiskSizeViewer, "Open Asset Disk Size Viewer", "Opens the Asset Disk Size Window", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowInContentBrowser, "Browse to Plugin", "Browses to the associated Plugin and selects it in the most recently used Content Browser (summoning one if necessary)", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::B));
	UI_COMMAND(ZoomToFit, "Zoom to Fit", "Zoom in and center the view on the selected item", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(ReCenterGraph, "Re-Center Graph", "Re-centers the graph on this node, showing all referencers and references for this Plugin instead", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE