// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

// Actions that can be invoked in the reference viewer
class FPluginReferenceViewerCommands : public TCommands<FPluginReferenceViewerCommands>
{
public:
	FPluginReferenceViewerCommands();

	// TCommands<> interface
	virtual void RegisterCommands() override;
	// End of TCommands<> interface

	TSharedPtr<FUICommandInfo> CompactMode;
	TSharedPtr<FUICommandInfo> ShowDuplicates;	
	TSharedPtr<FUICommandInfo> ShowEnginePlugins;
	TSharedPtr<FUICommandInfo> ShowOptionalPlugins;

	TSharedPtr<FUICommandInfo> OpenPluginProperties;
	TSharedPtr<FUICommandInfo> OpenAssetDiskSizeViewer;
	TSharedPtr<FUICommandInfo> ShowInContentBrowser;
	TSharedPtr<FUICommandInfo> ZoomToFit;
	TSharedPtr<FUICommandInfo> ReCenterGraph;
};