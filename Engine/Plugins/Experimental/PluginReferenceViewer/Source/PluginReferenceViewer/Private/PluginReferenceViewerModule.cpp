// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginReferenceViewerModule.h"

#include "EdGraphUtilities.h"
#include "EdGraphNode_PluginReference.h"
#include "Interfaces/IPluginManager.h"
#include "SPluginReferenceNode.h"
#include "SPluginReferenceViewer.h"
#include "Widgets/Docking/SDockTab.h"
#include "IPluginBrowser.h"

#define LOCTEXT_NAMESPACE "PluginReferenceViewer"

const FName FPluginReferenceViewerModule::PluginReferenceViewerTabName = TEXT("PluginReferenceViewer");

///////////////////////////////////////////

IMPLEMENT_MODULE(FPluginReferenceViewerModule, PluginReferenceViewer);

class FPluginReferenceViewerGraphPanelNodeFactory : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<SGraphNode> CreateNode(UEdGraphNode* Node) const override
	{
		if (UEdGraphNode_PluginReference* DependencyNode = Cast<UEdGraphNode_PluginReference>(Node))
		{
			return SNew(SPluginReferenceNode, DependencyNode);
		}

		return nullptr;
	}
};

void FPluginReferenceViewerModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(PluginReferenceViewerTabName, FOnSpawnTab::CreateRaw(this, &FPluginReferenceViewerModule::SpawnPluginReferenceViewerTab))
		.SetDisplayName(LOCTEXT("PluginReferenceViewerTitle", "Plugin Reference Viewer"))
		.SetMenuType(ETabSpawnerMenuType::Hidden);

	PluginReferenceViewerGraphPanelNodeFactory = MakeShareable(new FPluginReferenceViewerGraphPanelNodeFactory());
	FEdGraphUtilities::RegisterVisualNodeFactory(PluginReferenceViewerGraphPanelNodeFactory);

	IPluginBrowser::Get().OnLaunchReferenceViewerDelegate().BindRaw(this, &FPluginReferenceViewerModule::OnLaunchReferenceViewerFromPluginBrowser);
}

void FPluginReferenceViewerModule::ShutdownModule()
{
	IPluginBrowser::Get().OnLaunchReferenceViewerDelegate().Unbind();

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PluginReferenceViewerTabName);
}

TSharedRef<SDockTab> FPluginReferenceViewerModule::SpawnPluginReferenceViewerTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> WidgetReflectorTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	WidgetReflectorTab->SetContent(SNew(SPluginReferenceViewer)
		.ParentTab(WidgetReflectorTab));

	return WidgetReflectorTab;
}

void FPluginReferenceViewerModule::OnLaunchReferenceViewerFromPluginBrowser(TSharedPtr<IPlugin> Plugin)
{
	if (Plugin != nullptr)
	{
		OpenPluginReferenceViewerUI(Plugin.ToSharedRef());
	}
}

void FPluginReferenceViewerModule::OpenPluginReferenceViewerUI(const TSharedRef<class IPlugin>& Plugin)
{
	TArray<FPluginIdentifier> NewGraphRootNames;
	NewGraphRootNames.Add(FName(Plugin->GetName()));

	if (TSharedPtr<SDockTab> NewTab = FGlobalTabmanager::Get()->TryInvokeTab(PluginReferenceViewerTabName))
	{
		TSharedRef<SPluginReferenceViewer> PluginReferenceViewer = StaticCastSharedRef<SPluginReferenceViewer>(NewTab->GetContent());
		PluginReferenceViewer->SetGraphRootIdentifiers(NewGraphRootNames);
	}
}

#undef LOCTEXT_NAMESPACE
