// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginReferenceViewerSchema.h"

#include "AssetManagerEditorModule.h"
#include "AssetRegistry/AssetData.h"
#include "ConnectionDrawingPolicy.h"
#include "EdGraphNode_PluginReference.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/PathViews.h"
#include "PluginReferencePinCategory.h"
#include "PluginReferenceViewerCommands.h"
#include "PluginReferenceViewerModule.h"
#include "SPluginReferenceViewer.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PluginReferenceViewerSchema)

// Overridden connection drawing policy to use less curvy lines between nodes
class FReferenceViewerConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
public:
	FReferenceViewerConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements)
		: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements)
	{
	}

	virtual FVector2f ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const override
	{
		const int32 Tension = FMath::Abs<int32>(Start.X - End.X);
		return Tension * FVector2f(1.0f, 0);
	}

	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override
	{
		EPluginReferencePinCategory OutputCategory = PluginReferencePinUtil::ParseDependencyPinCategory(OutputPin->PinType.PinCategory);
		EPluginReferencePinCategory InputCategory = PluginReferencePinUtil::ParseDependencyPinCategory(InputPin->PinType.PinCategory);

		EPluginReferencePinCategory Category = !!(OutputCategory & EPluginReferencePinCategory::LinkEndActive) ? OutputCategory : InputCategory;
		Params.WireColor = PluginReferencePinUtil::GetColor(Category);
	}
};

//////////////////////////////////////////////////////////////////////////
// UReferenceViewerSchema

UPluginReferenceViewerSchema::UPluginReferenceViewerSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPluginReferenceViewerSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	const UEdGraphNode_PluginReference* ContextPluginNode = Cast<UEdGraphNode_PluginReference>(Context->Node);
	if (!ContextPluginNode->GetPlugin().IsValid())
	{
		return;
	}

	{
		FToolMenuSection& Section = Menu->AddSection(TEXT("Plugin"), NSLOCTEXT("PluginReferenceViewerSchema", "PluginSectionLabel", "Plugin"));
		
		Section.AddMenuEntry(FPluginReferenceViewerCommands::Get().OpenPluginProperties);

		Section.AddMenuEntry(FPluginReferenceViewerCommands::Get().OpenAssetDiskSizeViewer);
		
		FToolMenuEntry& ShowInEntry = Section.AddMenuEntry(FPluginReferenceViewerCommands::Get().ShowInContentBrowser);
		ShowInEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser.Small");
	}

	{
		FToolMenuSection& Section = Menu->AddSection(TEXT("AssetReferences"), NSLOCTEXT("PluginReferenceViewerSchema", "AssetReferencesSectionLabel", "Asset References"));

		FText Title = NSLOCTEXT("PluginReferenceViewerSchema", "OpenReferenceViewer", "Open {0} Asset References...");

		UEdGraph_PluginReferenceViewer* PluginReferenceViewer = const_cast<UEdGraph_PluginReferenceViewer*>(CastChecked<UEdGraph_PluginReferenceViewer>(Context->Graph));

		// Opens the reference viewer for all assets belonging to the plugin.
		check(!PluginReferenceViewer->GetCurrentGraphRootIdentifiers().IsEmpty());

		FPluginIdentifier GraphRootPluginIdentifier = PluginReferenceViewer->GetCurrentGraphRootIdentifiers()[0];

		if (ContextPluginNode->GetIdentifier() == GraphRootPluginIdentifier)
		{
			TArray<FAssetIdentifier> AssetIdentifiers;
			PluginReferenceViewer->GetPluginAssets(ContextPluginNode->GetIdentifier(), AssetIdentifiers);

			Section.AddEntry(FToolMenuEntry::InitMenuEntry(NAME_None,
				FText::Format(Title, FText::AsNumber(AssetIdentifiers.Num())),
				NSLOCTEXT("PluginReferenceViewerSchema", "PluginAssetReferences", "Launches the reference viewer showing the plugin asset references"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda(
					[AssetIdentifiers]()
					{
						IAssetManagerEditorModule::Get().OpenReferenceViewerUI(AssetIdentifiers);
					}),
					FCanExecuteAction::CreateLambda([AssetCount = AssetIdentifiers.Num()]() { return AssetCount > 0; })
				)
			));
		}
		else
		{
			/*	
			*	Opens the reference viewer for all assets that have references across plugins.
			*		a) from the root plugin node to the selected dependency plugin node.
			*		b) from the selected referencer plugin node to the root plugin node. 
			*/
			FText Tooltip = NSLOCTEXT("PluginReferenceViewerSchema", "AssetReferencesAcrossPlugins", "Launches the reference viewer showing the plugin asset references from '{0}' to '{1}'");
			TArray<FAssetIdentifier> AssetIdentifiers;
			FReferenceViewerParams ReferenceViewerParams;
			ReferenceViewerParams.bShowPluginFilter = true;

			if (PluginReferenceViewer->IsDependencyNode(ContextPluginNode->GetIdentifier()))
			{
				Tooltip = FText::Format(Tooltip, FText::FromName(GraphRootPluginIdentifier.Name), FText::FromName(ContextPluginNode->GetIdentifier().Name));

				ReferenceViewerParams.PluginFilter.Add(ContextPluginNode->GetIdentifier().Name);
				PluginReferenceViewer->FindAssetReferencesAcrossPlugins(GraphRootPluginIdentifier, ContextPluginNode->GetIdentifier(), AssetIdentifiers);
			}
			else
			{
				Tooltip = FText::Format(Tooltip, FText::FromName(ContextPluginNode->GetIdentifier().Name), FText::FromName(GraphRootPluginIdentifier.Name));

				ReferenceViewerParams.PluginFilter.Add(GraphRootPluginIdentifier.Name);
				PluginReferenceViewer->FindAssetReferencesAcrossPlugins(ContextPluginNode->GetIdentifier(), GraphRootPluginIdentifier, AssetIdentifiers);
			}

			Section.AddEntry(FToolMenuEntry::InitMenuEntry(NAME_None,
				FText::Format(Title, FText::AsNumber(AssetIdentifiers.Num())),
				Tooltip,
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda(
					[AssetIdentifiers, ReferenceViewerParams]()
					{
						IAssetManagerEditorModule::Get().OpenReferenceViewerUI(AssetIdentifiers, ReferenceViewerParams);
					}),
					FCanExecuteAction::CreateLambda([AssetCount=AssetIdentifiers.Num()]() { return AssetCount > 0; })
				)
			));
		}
	}

	{
		FToolMenuSection& Section = Menu->AddSection(TEXT("Misc"), NSLOCTEXT("PluginReferenceViewerSchema", "MiscSectionLabel", "Misc"));
		Section.AddMenuEntry(FPluginReferenceViewerCommands::Get().ZoomToFit);
		Section.AddMenuEntry(FPluginReferenceViewerCommands::Get().ReCenterGraph);
	}
}

FLinearColor UPluginReferenceViewerSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return PluginReferencePinUtil::GetColor(PluginReferencePinUtil::ParseDependencyPinCategory(PinType.PinCategory));
}

void UPluginReferenceViewerSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	// Don't allow breaking any links
}

void UPluginReferenceViewerSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	// Don't allow breaking any links
}

FPinConnectionResponse UPluginReferenceViewerSchema::MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove, bool bNotifyLinkedNodes) const
{
	// Don't allow moving any links
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, FString());
}

FPinConnectionResponse UPluginReferenceViewerSchema::CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy) const
{
	// Don't allow copying any links
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, FString());
}

FConnectionDrawingPolicy* UPluginReferenceViewerSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FReferenceViewerConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements);
}

void UPluginReferenceViewerSchema::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	if (TSharedPtr<IPlugin> Plugin = GetDroppedPlugin(Assets))
	{
		FPluginReferenceViewerModule::Get().OpenPluginReferenceViewerUI(MoveTemp(Plugin).ToSharedRef());
	}
}

void UPluginReferenceViewerSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	if (TSharedPtr<IPlugin> Plugin = GetDroppedPlugin(Assets))
	{
		OutTooltipText = FText::FormatOrdered(NSLOCTEXT("PluginReferenceViewerSchema", "RecenterGraphOnPlugin", "Re-center graph on plugin {0}."), FText::FromString(Plugin->GetFriendlyName())).ToString();
		OutOkIcon = true;
	}
	else
	{
		OutTooltipText = NSLOCTEXT("PluginReferenceViewerSchema", "NoPlugin", "Asset does not belong to a plugin.").ToString();
		OutOkIcon = false;
	}
}

TSharedPtr<IPlugin> UPluginReferenceViewerSchema::GetDroppedPlugin(const TArray<FAssetData>& Assets)
{
	for (const FAssetData& AssetData : Assets)
	{
		const FNameBuilder PackageName(AssetData.PackageName);
		FStringView PluginName = FPathViews::GetMountPointNameFromPath(PackageName);
		if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName))
		{
			return Plugin;
		}
	}

	return {};
}