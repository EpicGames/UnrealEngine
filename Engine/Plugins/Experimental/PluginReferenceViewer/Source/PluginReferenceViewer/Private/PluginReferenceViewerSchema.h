// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphSchema.h"
#include "PluginReferenceViewerSchema.generated.h"

class FSlateRect;
class IPlugin;
class UEdGraph;
class UToolMenu;

UCLASS(MinimalAPI)
class UPluginReferenceViewerSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphSchema interface
	virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual FName GetParentContextMenuName() const override { return NAME_None; }
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	virtual FPinConnectionResponse MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove = false, bool bNotifyLinkedNodes = false) const override;
	virtual FPinConnectionResponse CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy = false) const override;
	virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	// End of UEdGraphSchema interface

private:
	static TSharedPtr<IPlugin> GetDroppedPlugin(const TArray<FAssetData>& Assets);
};