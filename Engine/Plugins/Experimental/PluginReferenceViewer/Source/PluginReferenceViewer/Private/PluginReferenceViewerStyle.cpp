// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginReferenceViewerStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateTypes.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Misc/Paths.h"
#include "Styling/StyleColors.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"


FName FPluginReferenceViewerStyle::StyleName("PluginReferenceViewerStyle");

FPluginReferenceViewerStyle::FPluginReferenceViewerStyle()
	: FSlateStyleSet(StyleName)
{
	SetParentStyleName("EditorStyle");

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Editor/AssetManagerEditor/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FScrollBarStyle ScrollBar = GetParentStyle()->GetWidgetStyle<FScrollBarStyle>("ScrollBar");
	FTextBlockStyle NormalText = GetParentStyle()->GetWidgetStyle<FTextBlockStyle>("NormalText");
	FEditableTextBoxStyle NormalEditableTextBoxStyle = GetParentStyle()->GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");

	FSlateFontInfo TitleFont = DEFAULT_FONT("Bold", 12);

	// Text Styles 
	FTextBlockStyle GraphNodeTitle = FTextBlockStyle(NormalText)
		.SetFont(TitleFont)
		.SetColorAndOpacity( FStyleColors::White)
		.SetShadowOffset( FVector2D::UnitVector )
		.SetShadowColorAndOpacity( FLinearColor::Black );
	Set( "Graph.Node.NodeTitle", GraphNodeTitle );


	Set( "Graph.Node.NodeTitleExtraLines", FTextBlockStyle(NormalText)
		.SetFont( DEFAULT_FONT( "Normal", 9 ) )
		.SetColorAndOpacity( FStyleColors::White)
		.SetShadowOffset( FVector2D::ZeroVector)
		.SetShadowColorAndOpacity( FLinearColor::Transparent)
	);


	FEditableTextBoxStyle GraphNodeTitleEditableText = FEditableTextBoxStyle(NormalEditableTextBoxStyle)
		.SetFont(TitleFont)
		.SetForegroundColor(FStyleColors::Input)
		.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Foreground, FStyleColors::Secondary, 1.0f))
		.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Foreground, FStyleColors::Hover, 1.0f))
		.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Foreground, FStyleColors::Primary, 1.0f))
		.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, FStyleColors::InputOutline, 1.0f))
		.SetForegroundColor(FStyleColors::White)
		.SetBackgroundColor(FStyleColors::White)
		.SetReadOnlyForegroundColor(FStyleColors::Foreground)
		.SetFocusedForegroundColor(FStyleColors::Background)
		.SetScrollBarStyle( ScrollBar );
	Set( "Graph.Node.NodeTitleEditableText", GraphNodeTitleEditableText );

	Set( "Graph.Node.NodeTitleInlineEditableText", FInlineEditableTextBlockStyle()
		.SetTextStyle(GraphNodeTitle)
		.SetEditableTextBoxStyle(GraphNodeTitleEditableText)
	);

	
	FLinearColor SpillColor(.3f, .3f, .3f, 1.0f);
	int BodyRadius = 10.0; // Designed for 4 but using 10 to accomodate the shared selection border.  Update to 4 all nodes get aligned.

	// NOTE: 
	Set( "Graph.Node.BodyBackground", new FSlateRoundedBoxBrush(FStyleColors::Panel, BodyRadius));
	Set( "Graph.Node.BodyBorder",     new FSlateRoundedBoxBrush(SpillColor, BodyRadius));

	Set( "Graph.Node.Body",           new FSlateRoundedBoxBrush(FStyleColors::Panel, BodyRadius, FStyleColors::Transparent, 2.0));
	Set( "Graph.Node.ColorSpill",     new FSlateRoundedBoxBrush(SpillColor, FVector4(BodyRadius, BodyRadius, 0.0, 0.0)));

	Set( "Graph.Node.Duplicate",      new IMAGE_BRUSH_SVG("/GraphNode_Duplicate_8px", FVector2D(8.f, 8.f), FStyleColors::White));


	const FVector2D IconSize(20.0f, 20.0f);

	Set("Icons.ArrowLeft", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-left", IconSize));
	Set("Icons.ArrowRight", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-right", IconSize));

	Set("Icons.AutoFilters", new CORE_IMAGE_BRUSH_SVG("Starship/Common/FilterAuto", IconSize));
	Set("Icons.Filters", new CORE_IMAGE_BRUSH_SVG("Starship/Common/filter", IconSize));
	Set("Icons.Duplicate",      new IMAGE_BRUSH_SVG("/GraphNode_Duplicate_8px", IconSize, FStyleColors::White));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FPluginReferenceViewerStyle::~FPluginReferenceViewerStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FPluginReferenceViewerStyle& FPluginReferenceViewerStyle::Get()
{
	static FPluginReferenceViewerStyle Inst;
	return Inst;
}


