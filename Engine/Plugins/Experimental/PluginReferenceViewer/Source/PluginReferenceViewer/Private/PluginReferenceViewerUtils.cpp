// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginReferenceViewerUtils.h"

#include "Algo/Reverse.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "AssetManagerEditorModule.h"
#include "AssetRegistry/AssetData.h"
#include "DesktopPlatformModule.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsSettings.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"

#define LOCTEXT_NAMESPACE "PluginReferenceViewerUtils"

DEFINE_LOG_CATEGORY_STATIC(LogPluginReferenceViewerUtils, Log, All);

namespace PluginReferenceViewerUtils
{
	void ExportPlugins(const TArray<FString>& InArgs)
	{
		TArray<FString> PluginNames;
		if (InArgs.Num() >= 1)
		{
			InArgs[0].ParseIntoArray(PluginNames, TEXT(","));
		}
		else
		{
			UE_LOGF(LogPluginReferenceViewerUtils, Error, "Invalid plugin names argument. Expect plugin names separated by ',' as the first argument. e.g a,b,c");
			return;
		}

		FString Filename;
		if (InArgs.Num() >= 2)
		{
			Filename = InArgs[1];
		}
		else if (!PluginNames.IsEmpty())
		{
			Filename = FPaths::ProjectSavedDir() / FPaths::SetExtension(PluginNames[0], TEXT("csv"));
		}

		FPluginReferenceViewerUtils::ExportPlugins(PluginNames, Filename);
	}

	void ExportReference(const TArray<FString>& InArgs)
	{
		if (InArgs.Num() < 2)
		{
			UE_LOGF(LogPluginReferenceViewerUtils, Error, "Invalid arguments. Expected: [plugin name] [reference name] (optional)[filename.csv]");
			return;
		}

		FString PluginName = InArgs[0];
		FString ReferenceName = InArgs[1];

		FString Filename;
		if (InArgs.Num() >= 3)
		{
			Filename = InArgs[2];
		}
		else
		{
			Filename = FPaths::ProjectSavedDir() / FPaths::SetExtension(FString::Printf(TEXT("%s-%s"), *PluginName, *ReferenceName), TEXT("csv"));
		}

		FPluginReferenceViewerUtils::ExportReference(PluginName, ReferenceName, Filename);
	}
	
	void ExportAssetReference(const TArray<FString>& InArgs)
    	{
    		if (InArgs.Num() < 2)
    		{
    			UE_LOGF(LogPluginReferenceViewerUtils, Error, "Invalid arguments. Expected: [plugin name] [reference name] (optional)[filename.csv]");
    			return;
    		}
    
    		FString PluginName = InArgs[0];
    		FString ReferenceName = InArgs[1];
    
    		FString Filename;
    		if (InArgs.Num() >= 3)
    		{
    			Filename = InArgs[2];
    		}
    		else
    		{
    			Filename = FPaths::ProjectSavedDir() / FPaths::SetExtension(FString::Printf(TEXT("%s-%s"), *PluginName, *ReferenceName), TEXT("csv"));
    		}
    
    		FPluginReferenceViewerUtils::ExportAssetReference(PluginName, ReferenceName, Filename);
    	}

	void ExportDirectory(const TArray<FString>& InArgs)
	{
		TArray<FString> PluginNames;
		if (InArgs.Num() >= 1)
		{
			const FString SearchDirectory = FPaths::RootDir() / InArgs[0];
			if (FPaths::DirectoryExists(SearchDirectory))
			{
				TArray<FString> PluginFileNames;
				IPluginManager::Get().FindPluginsUnderDirectory(SearchDirectory, PluginFileNames);

				Algo::Transform(PluginFileNames, PluginNames, [](const FString& Item) { return FPaths::GetBaseFilename(Item); });
				PluginNames.Sort();
			}
			else
			{
				UE_LOGF(LogPluginReferenceViewerUtils, Error, "Directory does not exist %ls", *SearchDirectory);
				return;
			}
		}
		else
		{
			UE_LOGF(LogPluginReferenceViewerUtils, Error, "Invalid arguments. Expected directory path");
			return;
		}

		FString Filename;
		if (InArgs.Num() >= 2)
		{
			Filename = InArgs[1];
		}
		else if (!PluginNames.IsEmpty())
		{
			Filename = FPaths::ProjectSavedDir() / FPaths::SetExtension(FPaths::GetBaseFilename(InArgs[0]), TEXT("csv"));
		}

		FPluginReferenceViewerUtils::ExportPlugins(PluginNames, Filename);
	}

	static void GetPluginDependenciesRecursive_Helper(IPluginManager& PluginManager, const FString& ParentPluginName, TMap<FString, FString>& OutChildToParent)
	{
		TSharedPtr<IPlugin> Plugin = PluginManager.FindPlugin(ParentPluginName);
		if (Plugin.IsValid())
		{
			const FPluginDescriptor& Desc = Plugin->GetDescriptor();
			for (const FPluginReferenceDescriptor& Dependency : Desc.Plugins)
			{
				if (!OutChildToParent.Contains(Dependency.Name))
				{
					OutChildToParent.Add(Dependency.Name, ParentPluginName);
					GetPluginDependenciesRecursive_Helper(PluginManager, Dependency.Name, OutChildToParent);
				}
			}
		}
	}

	void GetPluginDependenciesRecursive(const FString& PluginName, TMap<FString, FString>& OutChildToParent)
	{
		GetPluginDependenciesRecursive_Helper(IPluginManager::Get(), PluginName, OutChildToParent);
	}

	bool TracePathFromPluginToPlugin(const FString& StartPluginName, const FString& EndPluginName, FString& FoundPathToEndPlugin)
	{
		const TSharedPtr<IPlugin> PluginPtr = IPluginManager::Get().FindPlugin(StartPluginName);
		if (!PluginPtr.IsValid())
		{
			UE_LOGF(LogPluginReferenceViewerUtils, Error, "Plugin `%ls` could not be found!", *StartPluginName);
			return false;
		}

		TMap<FString, FString> ChildToParent;
		GetPluginDependenciesRecursive(StartPluginName, ChildToParent);
		if (!ChildToParent.Contains(EndPluginName))
		{
			UE_LOGF(LogPluginReferenceViewerUtils, Display, "No paths from plugin `%ls` to plugin '%ls' was found!", *StartPluginName, *EndPluginName);
			return false;
		}

		TArray<FString> ReversePath;
		FString CurrentPluginName = EndPluginName;
		ReversePath.Add(CurrentPluginName);
		while (CurrentPluginName != StartPluginName)
		{
			CurrentPluginName = ChildToParent[CurrentPluginName];
			ReversePath.Add(CurrentPluginName);
		}
		Algo::Reverse(ReversePath); // Now correct path

		TStringBuilder<256> PathString;
		PathString.Appendf(TEXT("Found dependency path of length %d : "), ReversePath.Num());
		PathString.Join(ReversePath, TEXTVIEW(" -> "));
		FoundPathToEndPlugin = PathString;
		UE_LOGF(LogPluginReferenceViewerUtils, Display, "%ls", *FoundPathToEndPlugin);
		return true;
	}

	void TracePath(const TArray<FString>& InArgs)
	{
		FString StartPluginName;
		if (InArgs.Num() >= 1)
		{
			StartPluginName = InArgs[0];
		}
		else
		{
			UE_LOGF(LogPluginReferenceViewerUtils, Error, "Invalid arguments. Expected plugin name as 1st arg");
			return;
		}

		FString EndPluginName;
		if (InArgs.Num() >= 2)
		{
			EndPluginName = InArgs[1];
		}
		else
		{
			UE_LOGF(LogPluginReferenceViewerUtils, Error, "Invalid arguments. Expected plugin name as 2nd arg");
			return;
		}

		FString tmp;
		TracePathFromPluginToPlugin(StartPluginName, EndPluginName, tmp);
	}
}

namespace PluginReferenceViewerCVars
{
	static FAutoConsoleCommand ExportPlugins(
		TEXT("PluginReferenceViewer.ExportPlugins"),
		TEXT("Exports to .csv the number of references (by type) that a plugin has for each of it's dependencies.\n"
			"1st arg: single plugin name or multiple names separated with ','.\n"
			"2nd arg (optional): output filename.\n"
			"Example: PluginReferenceViewer.ExportPlugins PluginA,PluginB,PluginC PluginReport.csv"),
		FConsoleCommandWithArgsDelegate::CreateStatic(PluginReferenceViewerUtils::ExportPlugins)
	);
	
	static FAutoConsoleCommand ExportAssetReference(
    		TEXT("PluginReferenceViewer.ExportAssetReference"),
    		TEXT("Exports to .csv the list of asset references that exist between a plugin and one of it's dependencies, including which asset has the reference.\n"
    				"1st arg: plugin name.\n"
    				"2nd arg: reference name.\n"
    				"3rd arg (optional): output filename.\n"
    				"Example: PluginReferenceViewer.ExportAssetReference PluginName ReferenceName PluginReport.csv"),
    		FConsoleCommandWithArgsDelegate::CreateStatic(PluginReferenceViewerUtils::ExportAssetReference)
    	);

	static FAutoConsoleCommand ExportReference(
		TEXT("PluginReferenceViewer.ExportReference"),
		TEXT("Exports to .csv the list of asset references that exist between a plugin and one of it's dependencies.\n"
				"1st arg: plugin name.\n"
				"2nd arg: reference name.\n"
				"3rd arg (optional): output filename.\n"
				"Example: PluginReferenceViewer.ExportReference PluginName ReferenceName PluginReport.csv"),
		FConsoleCommandWithArgsDelegate::CreateStatic(PluginReferenceViewerUtils::ExportReference)
	);

	static FAutoConsoleCommand ExportDirectory(
		TEXT("PluginReferenceViewer.ExportDirectory"),
		TEXT("Exports to .csv the list of asset references that exist between each found plugin and all dependencies.\n"
			"1st arg: path relative to the root directory.\n"
			"2rd arg (optional): output filename.\n"
			"Example: PluginReferenceViewer.ExportDirectory Path PluginReport.csv"),
		FConsoleCommandWithArgsDelegate::CreateStatic(PluginReferenceViewerUtils::ExportDirectory)
	);

	static FAutoConsoleCommand TracePath(
		TEXT("PluginReferenceViewer.TracePath"),
		TEXT("Outputs all found plugin dependency paths from plugin X to plugin Y.\n"
			"1st arg: plugin start point name.\n"
			"2rd arg: plugin end point name.\n"
			"Example: PluginReferenceViewer.TracePath PluginX PluginY"),
		FConsoleCommandWithArgsDelegate::CreateStatic(PluginReferenceViewerUtils::TracePath)
	);
}

/*static*/TMap<FAssetIdentifier, TArray<FAssetIdentifier>> FPluginReferenceViewerUtils::GetAssetDependencyMap(const TSharedRef<IPlugin>& InPlugin)
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	TMap<FAssetIdentifier, TArray<FAssetIdentifier>> AssetDependencies;

	FARFilter Filter;
	Filter.bRecursivePaths = true;
	Filter.PackagePaths.Add(FName(InPlugin->GetMountedAssetPath()));

	TArray<FAssetData> AssetsInPlugin;
	AssetRegistry.GetAssets(Filter, AssetsInPlugin);

	TSet<FAssetIdentifier> UniqueDependencies;

	for (const FAssetData& AssetData : AssetsInPlugin)
	{
		TArray<FAssetIdentifier> AssetReferencers;
		AssetRegistry.GetDependencies(AssetData.PackageName, AssetReferencers);
		AssetDependencies.Add(FAssetIdentifier(AssetData.PackageName), AssetReferencers);

		for (const FAssetIdentifier& AssetReferencer : AssetReferencers)
		{
			UniqueDependencies.Add(AssetReferencer);
		}
	}

	return AssetDependencies;
}

/*static*/TArray<FAssetIdentifier> FPluginReferenceViewerUtils::GetAssetDependencies(const TSharedRef<IPlugin>& InPlugin)
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	FARFilter Filter;
	Filter.bRecursivePaths = true;
	Filter.PackagePaths.Add(FName(InPlugin->GetMountedAssetPath()));

	TArray<FAssetData> AssetsInPlugin;
	AssetRegistry.GetAssets(Filter, AssetsInPlugin);

	TSet<FAssetIdentifier> UniqueDependencies;

	for (const FAssetData& AssetData : AssetsInPlugin)
	{
		TArray<FAssetIdentifier> AssetReferencers;
		AssetRegistry.GetDependencies(AssetData.PackageName, AssetReferencers);

		for (const FAssetIdentifier& AssetReferencer : AssetReferencers)
		{
			UniqueDependencies.Add(AssetReferencer);
		}
	}

	return UniqueDependencies.Array();
}

/*static*/ TMap<FString, TArray<FAssetIdentifier>> FPluginReferenceViewerUtils::SplitByPlugins(const TSharedRef<IPlugin>& InOwningPlugin, const TArray<FAssetIdentifier>& InPluginDependencies)
{
	TMap<FString, TArray<FAssetIdentifier>> Results;

	const FName GameplayTagStructPackage = FGameplayTag::StaticStruct()->GetOutermost()->GetFName();
	const FName NAME_GameplayTag = FGameplayTag::StaticStruct()->GetFName();

	for (const FAssetIdentifier& AssetIdentifier : InPluginDependencies)
	{
		const FString PackageNameString = AssetIdentifier.PackageName.ToString();

		if (AssetIdentifier.ObjectName == NAME_GameplayTag && AssetIdentifier.PackageName == GameplayTagStructPackage)
		{
			const TArray<TSharedRef<IPlugin>> SourcePlugins = FPluginReferenceViewerUtils::FindGameplayTagSourcePlugins(AssetIdentifier.ValueName);
			for (const TSharedRef<IPlugin>& SourcePlugin : SourcePlugins)
			{
				if (SourcePlugin != InOwningPlugin)
				{
					Results.FindOrAdd(SourcePlugin->GetName()).Add(AssetIdentifier);
				}
			}
		}
		else if (FStringView ModuleName; FPackageName::TryConvertScriptPackageNameToModuleName(PackageNameString, ModuleName))
		{
			const TSharedPtr<IPlugin> ModulePlugin = IPluginManager::Get().GetModuleOwnerPlugin(FName(ModuleName));
			if (ModulePlugin.IsValid() && ModulePlugin != InOwningPlugin)
			{
				Results.FindOrAdd(ModulePlugin->GetName()).Add(AssetIdentifier);
			}
		}
		else
		{
			const TSharedPtr<IPlugin> PackagePlugin = IPluginManager::Get().FindPluginFromPath(PackageNameString);
			if (PackagePlugin.IsValid() && PackagePlugin != InOwningPlugin)
			{
				Results.FindOrAdd(PackagePlugin->GetName()).Add(AssetIdentifier);
			}
		}
	}

	return Results;
}

/*static*/ void FPluginReferenceViewerUtils::SplitByReferenceType(const TArray<FAssetIdentifier>& InAssetIdentifiers, TArray<FAssetIdentifier>& OutAssetReferences, TArray<FAssetIdentifier>& OutScriptReferences, TArray<FAssetIdentifier>& OutNameReferences)
{
	const FName GameplayTagStructPackage = FGameplayTag::StaticStruct()->GetOutermost()->GetFName();
	const FName NAME_GameplayTag = FGameplayTag::StaticStruct()->GetFName();
	const FName NAME_DataTable(TEXT("DataTable"));

	TArray<FName> AllPackageNames;
	for (const FAssetIdentifier& AssetIdentifier : InAssetIdentifiers)
	{
		AllPackageNames.Add(AssetIdentifier.PackageName);
	}

	TMap<FName, FAssetData> PackagesToAssetDataMap;
	UE::AssetRegistry::GetAssetForPackages(AllPackageNames, PackagesToAssetDataMap);

	for (const FAssetIdentifier& AssetIdentifier : InAssetIdentifiers)
	{
		const FString PackageNameString = AssetIdentifier.PackageName.ToString();

		if (AssetIdentifier.ObjectName == NAME_GameplayTag && AssetIdentifier.PackageName == GameplayTagStructPackage)
		{
			OutNameReferences.Add(AssetIdentifier);
		}
		else if (FPackageName::IsScriptPackage(PackageNameString))
		{
			FStringView ModuleName;
			if (FPackageName::TryConvertScriptPackageNameToModuleName(PackageNameString, ModuleName))
			{
				OutScriptReferences.Add(AssetIdentifier);
			}
		}
		else
		{
			// PackagesToAssetDataMap may not contain the package as references to assets that no longer exist do occur.
			const FAssetData* AssetData = PackagesToAssetDataMap.Find(AssetIdentifier.PackageName);
			if (AssetData != nullptr)
			{
				if (AssetData->AssetClassPath.GetAssetName() == NAME_DataTable)
				{
					OutNameReferences.Add(AssetIdentifier);
				}
				else
				{
					OutAssetReferences.Add(AssetIdentifier);
				}
			}
			else
			{
				UE_LOGF(LogPluginReferenceViewerUtils, Display, "Skipping package '%ls' due to missing asset data. Package may no longer exist!", *AssetIdentifier.PackageName.ToString());
			}
		}
	}
}

/*static*/ void FPluginReferenceViewerUtils::ExportPlugins(const TArray<FString>& InPluginNames, const FString& InFilename)
{
	TArray<TSharedRef<IPlugin>> Plugins;
	for (const FString& PluginName : InPluginNames)
	{
		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
		if (Plugin.IsValid())
		{
			Plugins.Add(Plugin.ToSharedRef());
		}
	}

	if (Plugins.IsEmpty())
	{
		UE_LOGF(LogPluginReferenceViewerUtils, Error, "Plugin names array is empty");
		return;
	}

	if (FPaths::GetExtension(InFilename) != TEXT("csv"))
	{
		UE_LOGF(LogPluginReferenceViewerUtils, Error, "Invalid filename extension '%ls'. Expected .csv", *InFilename);
		return;
	}

	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	AssetRegistry->WaitForCompletion();

	TUniquePtr<IFileHandle> ExportFileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*InFilename));
	if (!ExportFileHandle.IsValid())
	{
		return;
	}

	UTF16CHAR BOM = UNICODE_BOM;
	ExportFileHandle->Write((uint8*)&BOM, sizeof(UTF16CHAR));

	constexpr TCHAR Separator = TEXT(',');
	constexpr TCHAR LineEnd = TEXT('\n');

	TStringBuilder<1024> StringBuilder;
	
	StringBuilder.Append(TEXT("Plugin, Dependency, Enabled, Optional, Asset References, Script References, Name References, Total References"));
	StringBuilder.AppendChar(LineEnd);

	ExportFileHandle->Write((const uint8*)StringBuilder.ToString(), StringBuilder.Len() * sizeof(TCHAR));
	StringBuilder.Reset();

	FScopedSlowTask SlowTask(Plugins.Num(), LOCTEXT("Exporting Plugin Graph", "Exporting Plugin Graph..."));
	SlowTask.MakeDialog(true);

	for (const TSharedRef<IPlugin>& CurrentPlugin : Plugins)
	{
		SlowTask.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("ExportPluginName", "Processing plugin {0}"), FText::FromString(CurrentPlugin->GetName())));

		const TArray<FAssetIdentifier> AllDependencies = FPluginReferenceViewerUtils::GetAssetDependencies(CurrentPlugin);
		const TMap<FString, TArray<FAssetIdentifier>> PluginAssetMap = FPluginReferenceViewerUtils::SplitByPlugins(CurrentPlugin, AllDependencies);

		for (const FPluginReferenceDescriptor& ReferenceDescriptor : CurrentPlugin->GetDescriptor().Plugins)
		{
			// There might not be any asset references in the plugin dependency.
			const TArray<FAssetIdentifier>* PluginAssets = PluginAssetMap.Find(ReferenceDescriptor.Name);
			if (PluginAssets == nullptr)
			{
				continue;
			}

			TArray<FAssetIdentifier> AssetReferences, ScriptReferences, NameReferences;
			FPluginReferenceViewerUtils::SplitByReferenceType(*PluginAssets, AssetReferences, ScriptReferences, NameReferences);

			StringBuilder.Reset();

			StringBuilder.Append(CurrentPlugin->GetName());
			StringBuilder.AppendChar(Separator);

			StringBuilder.Append(ReferenceDescriptor.Name);
			StringBuilder.AppendChar(Separator);

			StringBuilder.Append(ReferenceDescriptor.bEnabled ? TEXT("true") : TEXT("false"));
			StringBuilder.AppendChar(Separator);

			StringBuilder.Append(ReferenceDescriptor.bOptional ? TEXT("true") : TEXT("false"));
			StringBuilder.AppendChar(Separator);

			StringBuilder.Appendf(TEXT("%d"), AssetReferences.Num());
			StringBuilder.AppendChar(Separator);

			StringBuilder.Appendf(TEXT("%d"), ScriptReferences.Num());
			StringBuilder.AppendChar(Separator);

			StringBuilder.Appendf(TEXT("%d"), NameReferences.Num());
			StringBuilder.AppendChar(Separator);

			StringBuilder.Appendf(TEXT("%d"), (*PluginAssets).Num());
			StringBuilder.AppendChar(Separator);

			StringBuilder.AppendChar(LineEnd);
			ExportFileHandle->Write((const uint8*)StringBuilder.ToString(), StringBuilder.Len() * sizeof(TCHAR));
		}
	}

	ExportFileHandle->Flush();


	TStringBuilder<256> ConcatenatePluginNames;
	ConcatenatePluginNames.Join(InPluginNames, TEXTVIEW(", "));
	UE_LOGF(LogPluginReferenceViewerUtils, Display, "Exported plugins; '%ls' to '%ls'", *ConcatenatePluginNames, *FPaths::ConvertRelativePathToFull(InFilename));
}

/*static*/ void FPluginReferenceViewerUtils::ExportAssetReference(const FString& InPlugin, const FString& InReference, const FString& InFilename)
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(InPlugin);
	if (!Plugin.IsValid())
	{
		UE_LOGF(LogPluginReferenceViewerUtils, Error, "Plugin %ls was not found!", *InPlugin);
		return;
	}

	const FPluginReferenceDescriptor* Found = Plugin->GetDescriptor().Plugins.FindByPredicate([=](const FPluginReferenceDescriptor& Item) { return Item.Name == InReference; });
	if (Found == nullptr)
	{
		UE_LOGF(LogPluginReferenceViewerUtils, Error, "Plugin reference %ls was not found!", *InReference);
		return;
	}

	if (FPaths::GetExtension(InFilename) != TEXT("csv"))
	{
		UE_LOGF(LogPluginReferenceViewerUtils, Error, "Invalid filename extenstion '%ls'. Expected .csv", *InFilename);
		return;
	}

	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	AssetRegistry->WaitForCompletion();

	TUniquePtr<IFileHandle> ExportFileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*InFilename));
	if (!ExportFileHandle.IsValid())
	{
		return;
	}

	UTF16CHAR BOM = UNICODE_BOM;
	ExportFileHandle->Write((uint8*)&BOM, sizeof(UTF16CHAR));

	constexpr TCHAR Separator = TEXT(',');
	constexpr TCHAR LineEnd = TEXT('\n');

	TStringBuilder<1024> StringBuilder;

	const TMap<FAssetIdentifier, TArray<FAssetIdentifier>> AllDependencies = FPluginReferenceViewerUtils::GetAssetDependencyMap(Plugin.ToSharedRef());
	for (const TPair<FAssetIdentifier, TArray<FAssetIdentifier>>& AssetDependency : AllDependencies)
	{
		const TMap<FString, TArray<FAssetIdentifier>> PluginAssetMap = FPluginReferenceViewerUtils::SplitByPlugins(Plugin.ToSharedRef(), AssetDependency.Value);
    
    	// There might not be any asset references in the plugin dependency.
    	const TArray<FAssetIdentifier>* PluginAssets = PluginAssetMap.Find(InReference);
    	if (PluginAssets != nullptr)
    	{
    		TArray<FAssetIdentifier> AssetReferences, ScriptReferences, NameReferences;
    		FPluginReferenceViewerUtils::SplitByReferenceType(*PluginAssets, AssetReferences, ScriptReferences, NameReferences);
    
    		for (TArray<FAssetIdentifier>* AssetList : { &AssetReferences, &ScriptReferences, &NameReferences })
    		{
    			for (const FAssetIdentifier& Identifier : *AssetList)
    			{
    				StringBuilder.Reset();
    				StringBuilder.Append(AssetDependency.Key.ToString());
    				StringBuilder.AppendChar(Separator);
    				StringBuilder.Append(Identifier.ToString());
    				StringBuilder.AppendChar(LineEnd);
    				ExportFileHandle->Write((const uint8*)StringBuilder.ToString(), StringBuilder.Len() * sizeof(TCHAR));
    			}
    		}
    	}
	}

	ExportFileHandle->Flush();
}

/*static*/ void FPluginReferenceViewerUtils::ExportReference(const FString& InPlugin, const FString& InReference, const FString& InFilename)
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(InPlugin);
	if (!Plugin.IsValid())
	{
		UE_LOGF(LogPluginReferenceViewerUtils, Error, "Plugin %ls was not found!", *InPlugin);
		return;
	}

	const FPluginReferenceDescriptor* Found = Plugin->GetDescriptor().Plugins.FindByPredicate([=](const FPluginReferenceDescriptor& Item) { return Item.Name == InReference; });
	if (Found == nullptr)
	{
		UE_LOGF(LogPluginReferenceViewerUtils, Error, "Plugin reference %ls was not found!", *InReference);
		return;
	}

	if (FPaths::GetExtension(InFilename) != TEXT("csv"))
	{
		UE_LOGF(LogPluginReferenceViewerUtils, Error, "Invalid filename extenstion '%ls'. Expected .csv", *InFilename);
		return;
	}

	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	AssetRegistry->WaitForCompletion();

	TUniquePtr<IFileHandle> ExportFileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*InFilename));
	if (!ExportFileHandle.IsValid())
	{
		return;
	}

	UTF16CHAR BOM = UNICODE_BOM;
	ExportFileHandle->Write((uint8*)&BOM, sizeof(UTF16CHAR));

	constexpr TCHAR Separator = TEXT(',');
	constexpr TCHAR LineEnd = TEXT('\n');

	TStringBuilder<1024> StringBuilder;

	const TArray<FAssetIdentifier> AllDependencies = FPluginReferenceViewerUtils::GetAssetDependencies(Plugin.ToSharedRef());
	const TMap<FString, TArray<FAssetIdentifier>> PluginAssetMap = FPluginReferenceViewerUtils::SplitByPlugins(Plugin.ToSharedRef(), AllDependencies);

	// There might not be any asset references in the plugin dependency.
	const TArray<FAssetIdentifier>* PluginAssets = PluginAssetMap.Find(InReference);
	if (PluginAssets != nullptr)
	{
		TArray<FAssetIdentifier> AssetReferences, ScriptReferences, NameReferences;
		FPluginReferenceViewerUtils::SplitByReferenceType(*PluginAssets, AssetReferences, ScriptReferences, NameReferences);

		for (TArray<FAssetIdentifier>* AssetList : { &AssetReferences, &ScriptReferences, &NameReferences })
		{
			for (const FAssetIdentifier& Identifier : *AssetList)
			{
				StringBuilder.Reset();

				StringBuilder.Append(Identifier.ToString());
				StringBuilder.AppendChar(LineEnd);
				ExportFileHandle->Write((const uint8*)StringBuilder.ToString(), StringBuilder.Len() * sizeof(TCHAR));
			}
		}
	}

	ExportFileHandle->Flush();
}

/*static*/ TArray<TSharedRef<IPlugin>> FPluginReferenceViewerUtils::FindGameplayTagSourcePlugins(FName TagName)
{
	TArray<TSharedRef<IPlugin>> Result;
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	FString Comment;
	TArray<FName> TagSources;
	bool bIsTagExplicit, bIsRestrictedTag, bAllowNonRestrictedChildren;
	if (Manager.GetTagEditorData(TagName, Comment, TagSources, bIsTagExplicit, bIsRestrictedTag, bAllowNonRestrictedChildren))
	{
		for (const FName& TagSourceName : TagSources)
		{
			if (const FGameplayTagSource* TagSource = Manager.FindTagSource(TagSourceName))
			{
				switch (TagSource->SourceType)
				{
				case EGameplayTagSourceType::TagList:
				{
					const FString ContentFilePath = FPaths::GetPath(TagSource->SourceTagList->ConfigFileName) / TEXT("../../Content/");
					FString RootContentPath;
					if (FPackageName::TryConvertFilenameToLongPackageName(ContentFilePath, RootContentPath))
					{
						if (const TSharedPtr<IPlugin> FoundPlugin = IPluginManager::Get().FindPluginFromPath(RootContentPath))
						{
							Result.Add(FoundPlugin.ToSharedRef());
						}
					}
					break;
				}
				case EGameplayTagSourceType::DataTable:
				{
					if (const TSharedPtr<IPlugin>& FoundPlugin = IPluginManager::Get().FindPluginFromPath(TagSource->SourceName.ToString()))
					{
						Result.Add(FoundPlugin.ToSharedRef());
					}
					break;
				}
				case EGameplayTagSourceType::Native:
				{
					if (const TSharedPtr<IPlugin>& FoundPlugin = IPluginManager::Get().GetModuleOwnerPlugin(TagSource->SourceName))
					{
						Result.Add(FoundPlugin.ToSharedRef());
					}
					break;
				}
				default:
					break;
				}
			}
		}
	}
	return Result;
}

bool FPluginReferenceViewerUtils::TracePluginChain(const FString& StartingPlugin, const FString& EndingPlugin, FString& OutPathToEndPlugin)
{
	return PluginReferenceViewerUtils::TracePathFromPluginToPlugin(StartingPlugin, EndingPlugin, OutPathToEndPlugin);
}

#undef LOCTEXT_NAMESPACE