// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPluginReferenceNode.h"

#include "EdGraph_PluginReferenceViewer.h"
#include "EdGraphNode_PluginReference.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "IDesktopPlatform.h"
#include "PluginReferenceViewerStyle.h"
#include "SCommentBubble.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "PluginReferenceViewer"

void SPluginReferenceNode::Construct(const FArguments& InArgs, UEdGraphNode_PluginReference* InNode)
{
	const int32 ThumbnailSize = 128;
	if (InNode->AllowsThumbnail())
	{
		// Plugin thumbnail image
		FString Icon128FilePath;
		if (InNode->GetPlugin().IsValid())
		{
			Icon128FilePath = InNode->GetPlugin()->GetBaseDir() / TEXT("Resources/Icon128.png");
		}
		if (!FPlatformFileManager::Get().GetPlatformFile().FileExists(*Icon128FilePath))
		{
			Icon128FilePath = IPluginManager::Get().FindPlugin(TEXT("PluginBrowser"))->GetBaseDir() / TEXT("Resources/DefaultIcon128.png");
		}

		const FName BrushName(*Icon128FilePath);
		const FIntPoint Size = FSlateApplication::Get().GetRenderer()->GenerateDynamicImageResource(BrushName);
		if ((Size.X > 0) && (Size.Y > 0))
		{
			PluginIconDynamicImageBrush = MakeShareable(new FSlateDynamicImageBrush(BrushName, FVector2D(ThumbnailSize, ThumbnailSize)));
		}
	}

	GraphNode = InNode;
	SetCursor(EMouseCursor::CardinalCross);
	UpdateGraphNode();
}

// UpdateGraphNode is similar to the base, but adds the option to hide the thumbnail */
void SPluginReferenceNode::UpdateGraphNode()
{
	OutputPins.Empty();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	UpdateErrorInfo();

	//
	//             ______________________
	//            |      TITLE AREA      |
	//            +-------+------+-------+
	//            | (>) L |      | R (>) |
	//            | (>) E |      | I (>) |
	//            | (>) F |      | G (>) |
	//            | (>) T |      | H (>) |
	//            |       |      | T (>) |
	//            |_______|______|_______|
	//
	TSharedPtr<SVerticalBox> MainVerticalBox;
	TSharedPtr<SErrorText> ErrorText;
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode).StyleSet(&FPluginReferenceViewerStyle::Get());

	// Get node icon
	IconColor = FLinearColor::White;
	const FSlateBrush* IconBrush = nullptr;
	if (GraphNode != NULL && GraphNode->ShowPaletteIconOnNode())
	{
		IconBrush = GraphNode->GetIconAndTint(IconColor).GetOptionalIcon();
	}

	TSharedRef<SWidget> ThumbnailWidget = SNullWidget::NullWidget;
	UEdGraphNode_PluginReference* RefGraphNode = CastChecked<UEdGraphNode_PluginReference>(GraphNode);
	bool bIsADuplicate = RefGraphNode->IsADuplicate();

	FLinearColor OpacityColor = /*RefGraphNode->GetIsFiltered() ? FLinearColor(1.0, 1.0, 1.0, 0.4) :*/FLinearColor::White;
	
	if (PluginIconDynamicImageBrush.IsValid())
	{
		ThumbnailWidget = SNew(SBox)
			.WidthOverride(PluginIconDynamicImageBrush->GetImageSize().X)
			.HeightOverride(PluginIconDynamicImageBrush->GetImageSize().Y)
			[
				SNew(SImage)
				.Image(PluginIconDynamicImageBrush.Get())
			];
	}

	GetOrAddSlot(ENodeZone::Center)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		SAssignNew(MainVerticalBox, SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.ColorAndOpacity(OpacityColor)
			.BorderImage(FPluginReferenceViewerStyle::Get().GetBrush("Graph.Node.BodyBackground"))
			.Padding(0)
			[
				SNew(SBorder)
				.BorderImage(FPluginReferenceViewerStyle::Get().GetBrush("Graph.Node.BodyBorder"))
				.BorderBackgroundColor(this, &SPluginReferenceNode::GetNodeTitleColor)
				.Padding(0)
				[
					SNew(SBorder)
					.BorderImage(FPluginReferenceViewerStyle::Get().GetBrush("Graph.Node.Body"))
					.Padding(0)
					[
						SNew(SVerticalBox)
						.ToolTipText(this, &SPluginReferenceNode::GetNodeTooltip)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Top)
						.Padding(0)
						[
							SNew(SBorder)
							.BorderImage(FPluginReferenceViewerStyle::Get().GetBrush("Graph.Node.ColorSpill"))
							.Padding(FMargin(10.0f, 4.0f, 6.0f, 4.0f))
							.BorderBackgroundColor(this, &SPluginReferenceNode::GetNodeTitleColor)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
								.AutoWidth()
								[
									SNew(SImage)
									.Image(IconBrush)
									.DesiredSizeOverride(FVector2D(24.0, 24.0))
									.ColorAndOpacity(this, &SGraphNode::GetNodeTitleIconColor)
								]
		
								+ SHorizontalBox::Slot()
								.FillWidth(1.0)
								.VAlign(VAlign_Center)
								[
									SNew(SVerticalBox)
									+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(FMargin(0.f))
									.VAlign(VAlign_Center)
									[
										SAssignNew(InlineEditableText, SInlineEditableTextBlock)
										.Style(FPluginReferenceViewerStyle::Get(), "Graph.Node.NodeTitleInlineEditableText")
										.Text(NodeTitle.Get(), &SNodeTitle::GetHeadTitle)
										.OnVerifyTextChanged(this, &SPluginReferenceNode::OnVerifyNameTextChanged)
										.OnTextCommitted(this, &SPluginReferenceNode::OnNameTextCommited)
										.IsReadOnly(this, &SPluginReferenceNode::IsNameReadOnly)
										.IsSelected(this, &SPluginReferenceNode::IsSelectedExclusively)
									]
									+ SVerticalBox::Slot()
									.AutoHeight()
									.Padding(FMargin(0.f))
									[
										NodeTitle.ToSharedRef()
									]
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Right)
								.VAlign(VAlign_Bottom)
								.Padding(FMargin(4.f, 0.f, 0.f, 1.f))
								[
									SNew(SImage)
									.Visibility(bIsADuplicate ? EVisibility::Visible : EVisibility::Hidden)
									.ToolTipText(LOCTEXT("DuplicateAsset", "This plugin is referenced multiple times. Only the first occurance shows its descendants."))
									.DesiredSizeOverride(FVector2D(12.0, 12.0))
									.Image(FPluginReferenceViewerStyle::Get().GetBrush("Graph.Node.Duplicate"))
									.ColorAndOpacity(FAppStyle::Get().GetColor("Colors.Foreground"))
								]
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(1.0f)
						[
							// POPUP ERROR MESSAGE
							SAssignNew(ErrorText, SErrorText)
							.BackgroundColor(this, &SPluginReferenceNode::GetErrorColor)
							.ToolTipText(this, &SPluginReferenceNode::GetErrorMsgToolTip)
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Top)
						[
							// NODE CONTENT AREA
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("NoBorder"))
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							.Padding(FMargin(0, 3))
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									// LEFT
									SNew(SBox)
									.WidthOverride(40)
									[
										SAssignNew(LeftNodeBox, SVerticalBox)
									]
								]

								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								.FillWidth(1.0f)
								[
									ThumbnailWidget
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									// RIGHT
									SNew(SBox)
									.WidthOverride(40)
									[
										SAssignNew(RightNodeBox, SVerticalBox)
									]
								]
							]
						]

					] // Inner Background 
				] // Outline Border
			] // Background 
		]
	];

	// Create comment bubble if comment text is valid
	GetNodeObj()->bCommentBubbleVisible = !GetNodeObj()->NodeComment.IsEmpty();
	if (GetNodeObj()->ShouldMakeCommentBubbleVisible() && GetNodeObj()->bCommentBubbleVisible)
	{
		TSharedPtr<SCommentBubble> CommentBubble;

		SAssignNew(CommentBubble, SCommentBubble)
			.GraphNode(GraphNode)
			.Text(this, &SGraphNode::GetNodeComment);

		GetOrAddSlot(ENodeZone::TopCenter)
			.SlotOffset2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetOffset2f))
			.SlotSize2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetSize2f))
			.AllowScaling(TAttribute<bool>(CommentBubble.Get(), &SCommentBubble::IsScalingAllowed))
			.VAlign(VAlign_Top)
			[
				CommentBubble.ToSharedRef()
			];
	}

	ErrorReporting = ErrorText;
	ErrorReporting->SetError(ErrorMsg);
	CreateBelowWidgetControls(MainVerticalBox);

	CreatePinWidgets();
}

#undef LOCTEXT_NAMESPACE
