// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"

class FAssetThumbnail;
class UEdGraphNode_PluginReference;
struct FSlateDynamicImageBrush;

/**
 *
 */
class SPluginReferenceNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SPluginReferenceNode) {}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, UEdGraphNode_PluginReference* InNode);

	// SGraphNode implementation
	virtual void UpdateGraphNode() override;
	virtual bool IsNodeEditable() const override { return false; }
	// End SGraphNode implementation

private:
	TSharedPtr<class FAssetThumbnail> AssetThumbnail;

	/** Brush resource for the image that is dynamically loaded */
	TSharedPtr<FSlateDynamicImageBrush> PluginIconDynamicImageBrush;
};