// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPluginReferenceViewer.h"

#include "AssetManagerEditorModule.h"
#include "ContentBrowserModule.h"
#include "EdGraphNode_PluginReference.h"
#include "EdGraph_PluginReferenceViewer.h"
#include "IContentBrowserDataModule.h"
#include "IContentBrowserSingleton.h"
#include "Features/EditorFeatures.h"
#include "Features/IModularFeatures.h"
#include "Features/IPluginsEditorFeature.h"
#include "Framework/Docking/TabManager.h"
#include "Interfaces/IPluginManager.h"
#include "Math/UnitConversion.h"
#include "PluginReferenceViewerCommands.h"
#include "PluginReferenceViewerSchema.h"
#include "PluginReferenceViewerStyle.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"

#define LOCTEXT_NAMESPACE "PluginReferenceViewer"

namespace PluginReferenceViewerTabID
{
	static const FName ReferenceGraph = "PluginReferenceViewer.ReferenceGraph";
	static const FName PluginStats = "PluginReferenceViewer.PluginStats";
}

namespace
{
	/** Provided a byte count this function outputs best fit representation of size using FUnitConversion
	* @param SizeInBytes The amount of bytes to convert to a formatted text that represents it.
	* @return A text 
	*/
	FText GetSizeInBytesAsBiggerUnits(const uint64 SizeInBytes)
	{
		const FNumericUnit<uint64> Unit = FUnitConversion::QuantizeUnitsToBestFit(SizeInBytes, EUnit::Bytes);

		FString UnitAsString = FString::Printf(TEXT("%lld %s"), Unit.Value, FUnitConversion::GetUnitDisplayString(Unit.Units));
		
		return FText::FromString(UnitAsString);
	}
}


SPluginReferenceViewer::~SPluginReferenceViewer()
{
	if (!GExitPurge)
	{
		if (ensure(GraphObj))
		{
			GraphObj->RemoveFromRoot();
		}
	}
}

void SPluginReferenceViewer::Construct(const FArguments& InArgs)
{
	/** Used to delay graph rebuilding during spinbox slider interaction */
	bNeedsGraphRebuild = false;

	/** Enables loading of plugin stats, such as reference counts, plugin content size, and dependant content size. */
	bShowAdvancedInfo = false;

	RegisterActions();

	// Set up the history manager
	HistoryManager.SetOnApplyHistoryData(FOnApplyHistoryData::CreateSP(this, &SPluginReferenceViewer::OnApplyHistoryData));
	HistoryManager.SetOnUpdateHistoryData(FOnUpdateHistoryData::CreateSP(this, &SPluginReferenceViewer::OnUpdateHistoryData));

	// Create the graph
	GraphObj = NewObject<UEdGraph_PluginReferenceViewer>();
	GraphObj->Schema = UPluginReferenceViewerSchema::StaticClass();
	GraphObj->AddToRoot();
	GraphObj->SetPluginReferenceViewer(StaticCastSharedRef<SPluginReferenceViewer>(AsShared()));

	SGraphEditor::FGraphEditorEvents GraphEvents;
	GraphEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &SPluginReferenceViewer::OnNodeDoubleClicked);

	// Create the graph editor
	GraphEditorPtr = SNew(SGraphEditor)
		.AdditionalCommands(PluginReferenceViewerActions)
		.GraphToEdit(GraphObj)
		.GraphEvents(GraphEvents)
		.ShowGraphStateOverlay(false)
		.OnNavigateHistoryBack(FSimpleDelegate::CreateSP(this, &SPluginReferenceViewer::GraphNavigateHistoryBack))
		.OnNavigateHistoryForward(FSimpleDelegate::CreateSP(this, &SPluginReferenceViewer::GraphNavigateHistoryForward));

	GraphObj->CachePluginDependencies(IPluginManager::Get().GetDiscoveredPlugins());


	const FName TabLayoutName = "PluginReferenceViewer_Layout_NoStats";

	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout(TabLayoutName)
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Vertical)
		->Split
		(
			// Main application area
			FTabManager::NewSplitter()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->SetSizeCoefficient(0.7f)
				->AddTab(PluginReferenceViewerTabID::ReferenceGraph, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetHideTabWell(true)
				->SetSizeCoefficient(0.3f)
				->AddTab(PluginReferenceViewerTabID::PluginStats, ETabState::ClosedTab)
			)
		)
	);

	auto RegisterTrackedTabSpawner = [this](const FName& TabId, const FOnSpawnTab& OnSpawnTab) -> FTabSpawnerEntry&
	{
		return TabManager->RegisterTabSpawner(TabId, FOnSpawnTab::CreateLambda([this, OnSpawnTab](const FSpawnTabArgs& Args) -> TSharedRef<SDockTab>
			{
				TSharedRef<SDockTab> SpawnedTab = OnSpawnTab.Execute(Args);
				OnTabSpawned(Args.GetTabId().TabType, SpawnedTab);
				return SpawnedTab;
			}));
	};

	check(InArgs._ParentTab.IsValid());
	TabManager = FGlobalTabmanager::Get()->NewTabManager(InArgs._ParentTab.ToSharedRef());

	RegisterTrackedTabSpawner(PluginReferenceViewerTabID::ReferenceGraph, FOnSpawnTab::CreateSP(this, &SPluginReferenceViewer::SpawnTab_ReferenceGraph))
		.SetDisplayName(LOCTEXT("ReferenceGraphTab", "Reference Graph"));

	RegisterTrackedTabSpawner(PluginReferenceViewerTabID::PluginStats, FOnSpawnTab::CreateSP(this, &SPluginReferenceViewer::SpawnTab_PluginStats))
		.SetDisplayName(LOCTEXT("PluginStatsTab", "Plugin Stats"));

	ChildSlot
	[
		TabManager->RestoreFrom(Layout, nullptr).ToSharedRef()
	];
}

void SPluginReferenceViewer::SetGraphRootIdentifiers(const TArray<FPluginIdentifier>& NewGraphRootIdentifiers)
{
	GraphObj->SetGraphRoot(NewGraphRootIdentifiers);

	GraphObj->RebuildGraph();

	GraphEditorPtr->ZoomToFit(false);

	// Set the initial history data
	HistoryManager.AddHistoryData();

	TemporaryPathBeingEdited = NewGraphRootIdentifiers.Num() > 0 ? FText() : FText(LOCTEXT("NoPluginsFound", "No Plugins Found"));
}

void SPluginReferenceViewer::OnOpenPluginProperties()
{
	TSharedPtr<const IPlugin> Plugin;

	const FGraphPanelSelectionSet& SelectedNodes = GraphEditorPtr->GetSelectedNodes();
	if (!SelectedNodes.IsEmpty())
	{
		for (UObject* Node : SelectedNodes)
		{
			UEdGraphNode_PluginReference* PluginReferenceNode = Cast<UEdGraphNode_PluginReference>(Node);
			if (PluginReferenceNode)
			{
				Plugin = PluginReferenceNode->GetPlugin();
				break;
			}
		}
	}

	if (Plugin != nullptr)
	{
		OpenPluginProperties(Plugin->GetName());
	}
}

void SPluginReferenceViewer::OnOpenAssetDiskSizeViewer()
{
	if (UEdGraphNode* SelectedNode = GraphEditorPtr->GetSingleSelectedNode())
	{
		if (UEdGraphNode_PluginReference* PluginReferenceNode = Cast<UEdGraphNode_PluginReference>(SelectedNode))
		{
			if (TSharedPtr<const IPlugin> PluginReferenced = PluginReferenceNode->GetPlugin())
			{
				IAssetManagerEditorModule::Get().OpenAssetSizeTreeViewUI(PluginReferenced->GetName());
			}
		}
	}
}

void SPluginReferenceViewer::OnShowPluginInContentBrowser()
{
	TSharedPtr<const IPlugin> Plugin;

	const FGraphPanelSelectionSet& SelectedNodes = GraphEditorPtr->GetSelectedNodes();
	if (!SelectedNodes.IsEmpty())
	{
		for (UObject* Node : SelectedNodes)
		{
			UEdGraphNode_PluginReference* PluginReferenceNode = Cast<UEdGraphNode_PluginReference>(Node);
			if (PluginReferenceNode)
			{
				Plugin = PluginReferenceNode->GetPlugin();
				break;
			}
		}
	}

	if (Plugin != nullptr)
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

		FString MountedPath = Plugin->GetMountedAssetPath();
		FStringView MountedPathView(MountedPath);
		
		// Remove any trailing slashes
		if (MountedPathView.EndsWith('/'))
		{
			MountedPathView.LeftChopInline(1);
		}

		FName VirtualPath;
		ContentBrowserData->ConvertInternalPathToVirtual(MountedPathView, VirtualPath);

		if (!VirtualPath.IsNone())
		{
			FContentBrowserItem Item = ContentBrowserData->GetItemAtPath(VirtualPath, EContentBrowserItemTypeFilter::IncludeAll);
			if (Item.IsValid())
			{
				TArray<FContentBrowserItem> Items = { Item };
				IContentBrowserSingleton& ContentBrowser = FModuleManager::Get().GetModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
				ContentBrowser.SyncBrowserToItems(Items);
			}	
		}
	}
}

void SPluginReferenceViewer::OnApplyHistoryData(const FPluginReferenceViewerHistoryData& History)
{
	if (GraphObj)
	{
		GraphObj->SetGraphRoot(History.Identifiers);
		UEdGraphNode_PluginReference* NewRootNode = GraphObj->RebuildGraph();

		if (NewRootNode && ensure(GraphEditorPtr.IsValid()))
		{
			GraphEditorPtr->SetNodeSelection(NewRootNode, true);
		}

		TemporaryPathBeingEdited = FText();
	}
}

void SPluginReferenceViewer::OnUpdateHistoryData(FPluginReferenceViewerHistoryData& HistoryData) const
{
	if (GraphObj)
	{
		const TArray<FPluginIdentifier>& CurrentGraphRootIdentifiers = GraphObj->GetCurrentGraphRootIdentifiers();
		HistoryData.HistoryDesc = GetAddressBarText();
		HistoryData.Identifiers = CurrentGraphRootIdentifiers;
	}
	else
	{
		HistoryData.HistoryDesc = FText::GetEmpty();
		HistoryData.Identifiers.Empty();
	}
}

bool SPluginReferenceViewer::HasAtLeastOneRealNodeSelected()
{
	// NOTE: Based on SReferenceViewer::HasAtLeastOneRealNodeSelected without need to check for collapsed node
	if (GraphEditorPtr.IsValid())
	{
		TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();
		for (UObject* Node : SelectedNodes)
		{
			UEdGraphNode_PluginReference* ReferenceNode = Cast<UEdGraphNode_PluginReference>(Node);
			if (ReferenceNode)
			{
				return true;
			}
		}
	}

	return false;
}

void SPluginReferenceViewer::OpenPluginProperties(const FString& PluginName)
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
	if (Plugin != nullptr)
	{
		IPluginsEditorFeature& PluginEditor = IModularFeatures::Get().GetModularFeature<IPluginsEditorFeature>(EditorFeatures::PluginsEditor);
		PluginEditor.OpenPluginEditor(Plugin.ToSharedRef(), nullptr, FSimpleDelegate());
	}
}

TSharedRef<SWidget> SPluginReferenceViewer::MakeToolBar()
{
	FToolBarBuilder ToolBarBuilder(PluginReferenceViewerActions, FMultiBoxCustomization::None, TSharedPtr<FExtender>(), true);

	ToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SPluginReferenceViewer::BackClicked),
			FCanExecuteAction::CreateSP(this, &SPluginReferenceViewer::IsBackEnabled)
		),
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>::CreateSP(this, &SPluginReferenceViewer::GetHistoryBackTooltip),
		FSlateIcon(FPluginReferenceViewerStyle::Get().GetStyleSetName(), "Icons.ArrowLeft"));

	ToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SPluginReferenceViewer::ForwardClicked),
			FCanExecuteAction::CreateSP(this, &SPluginReferenceViewer::IsForwardEnabled)
		),
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>::CreateSP(this, &SPluginReferenceViewer::GetHistoryForwardTooltip),
		FSlateIcon(FPluginReferenceViewerStyle::Get().GetStyleSetName(), "Icons.ArrowRight"));

	ToolBarBuilder.AddSeparator();

	ToolBarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &SPluginReferenceViewer::GetShowMenuContent),
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Visibility"),
		/*bInSimpleComboBox*/ false);
	
	return ToolBarBuilder.MakeWidget();
}

TSharedRef<SWidget> SPluginReferenceViewer::GetShowMenuContent()
{
	FMenuBuilder MenuBuilder(true, PluginReferenceViewerActions);

	MenuBuilder.BeginSection("Reference Types", LOCTEXT("ReferenceTypes", "Reference Types"));
	MenuBuilder.AddMenuEntry(FPluginReferenceViewerCommands::Get().ShowEnginePlugins);
	MenuBuilder.AddMenuEntry(FPluginReferenceViewerCommands::Get().ShowOptionalPlugins);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("ViewOptions", LOCTEXT("ViewOptions", "View Options"));
	MenuBuilder.AddMenuEntry(FPluginReferenceViewerCommands::Get().CompactMode);
	MenuBuilder.AddMenuEntry(FPluginReferenceViewerCommands::Get().ShowDuplicates);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SPluginReferenceViewer::RebuildGraph()
{
	GraphObj->RebuildGraph();
}

bool SPluginReferenceViewer::IsBackEnabled() const
{
	return HistoryManager.CanGoBack();
}

bool SPluginReferenceViewer::IsForwardEnabled() const
{
	return HistoryManager.CanGoForward();
}

void SPluginReferenceViewer::BackClicked()
{
	HistoryManager.GoBack();
}

void SPluginReferenceViewer::ForwardClicked()
{
	HistoryManager.GoForward();
}

void SPluginReferenceViewer::RefreshClicked()
{
	RebuildGraph();
	ZoomToFit();
}

void SPluginReferenceViewer::GraphNavigateHistoryBack()
{
	BackClicked();
}

void SPluginReferenceViewer::GraphNavigateHistoryForward()
{
	ForwardClicked();
}

FText SPluginReferenceViewer::GetHistoryBackTooltip() const
{
	if (HistoryManager.CanGoBack())
	{
		return FText::Format(LOCTEXT("HistoryBackTooltip", "Back to {0}"), HistoryManager.GetBackDesc());
	}
	return FText::GetEmpty();
}

FText SPluginReferenceViewer::GetHistoryForwardTooltip() const
{
	if (HistoryManager.CanGoForward())
	{
		return FText::Format(LOCTEXT("HistoryForwardTooltip", "Forward to {0}"), HistoryManager.GetForwardDesc());
	}
	return FText::GetEmpty();
}

FText SPluginReferenceViewer::GetPluginStatsText() const
{
	const UEdGraphNode* SelectedNode = GraphEditorPtr->GetSingleSelectedNode();

	if (SelectedNode == nullptr)
	{
		return FText(LOCTEXT("PluginSelectionError", "Invalid number of nodes selected. Please select a single node!"));
	}

	const UEdGraphNode_PluginReference* PluginReferenceNode = Cast<const UEdGraphNode_PluginReference>(SelectedNode);

	if (PluginReferenceNode == nullptr)
	{
		return FText::GetEmpty();
	}

	const FPluginStats & PluginStats = GraphObj->GetPluginStats(PluginReferenceNode->GetIdentifier());

	return FText::Format(LOCTEXT("PluginInfoText", "Name: {0}\nContent Size: {1}\nTotal Content Size With Dependencies: {2}\nDependencies: {3}\nReferencers: {4}"), 
		{ FText::FromString(PluginReferenceNode->GetIdentifier().ToString()), GetSizeInBytesAsBiggerUnits(PluginStats.Size), GetSizeInBytesAsBiggerUnits(PluginStats.SizeWithDependencies), PluginStats.Dependencies, PluginStats.Referencers });
}

FText SPluginReferenceViewer::GetAddressBarText() const
{
	if (GraphObj)
	{
		if (TemporaryPathBeingEdited.IsEmpty())
		{
			const TArray<FPluginIdentifier>& CurrentGraphRootIdentifiers = GraphObj->GetCurrentGraphRootIdentifiers();
			if (CurrentGraphRootIdentifiers.Num() == 1)
			{
				TSharedPtr<const IPlugin> Plugin = IPluginManager::Get().FindPlugin(CurrentGraphRootIdentifiers[0].ToString());
				FString PluginPath = Plugin->GetBaseDir();
				FPaths::MakePathRelativeTo(PluginPath, *FPaths::RootDir());

				return FText::FromString(PluginPath);
			}
			else if (CurrentGraphRootIdentifiers.Num() > 1)
			{
				return FText::Format(LOCTEXT("AddressBarMultiplePackagesText", "{0} and {1} others"), FText::FromString(CurrentGraphRootIdentifiers[0].ToString()), FText::AsNumber(CurrentGraphRootIdentifiers.Num()));
			}
		}
		else
		{
			return TemporaryPathBeingEdited;
		}
	}

	return FText();
}

void SPluginReferenceViewer::OnAddressBarTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		TArray<FPluginIdentifier> NewPaths;

		// Plugins are identified by their unique name. The rest of the path is not important but it is displayed
		// in the address bar for added clarity.
		FString PluginName = FPaths::GetCleanFilename(NewText.ToString());

		TSharedPtr<const IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
		if (Plugin != nullptr)
		{
			FPluginIdentifier NewPath = FPluginIdentifier::FromString(Plugin->GetName());
			NewPaths.Add(NewPath);
		}

		SetGraphRootIdentifiers(NewPaths);
	}
}

void SPluginReferenceViewer::OnAddressBarTextChanged(const FText& NewText)
{
	TemporaryPathBeingEdited = NewText;
}

void SPluginReferenceViewer::ZoomToFit()
{
	GraphEditorPtr->ZoomToFit(true);
}

void SPluginReferenceViewer::ReCenterGraph()
{
	ReCenterGraphOnNodes(GraphEditorPtr->GetSelectedNodes());
}

void SPluginReferenceViewer::ReCenterGraphOnNodes(const TSet<UObject*>& Nodes)
{
	TArray<FPluginIdentifier> NewGraphRootNames;
	FIntPoint TotalNodePos(ForceInitToZero);
	for (auto NodeIt = Nodes.CreateConstIterator(); NodeIt; ++NodeIt)
	{
		UEdGraphNode_PluginReference* PluginReferenceNode = Cast<UEdGraphNode_PluginReference>(*NodeIt);
		if (PluginReferenceNode)
		{
			NewGraphRootNames.Add(PluginReferenceNode->GetIdentifier());
			TotalNodePos.X += PluginReferenceNode->NodePosX;
			TotalNodePos.Y += PluginReferenceNode->NodePosY;
		}
	}

	if (NewGraphRootNames.Num() > 0)
	{
		const FIntPoint AverageNodePos = TotalNodePos / NewGraphRootNames.Num();
		GraphObj->SetGraphRoot(NewGraphRootNames, AverageNodePos);
		UEdGraphNode_PluginReference* NewRootNode = GraphObj->RebuildGraph();

		if (NewRootNode != nullptr)
		{
			GraphEditorPtr->ClearSelectionSet();
			GraphEditorPtr->SetNodeSelection(NewRootNode, true);
		}

		// Set the initial history data
		HistoryManager.AddHistoryData();
	}
}

void SPluginReferenceViewer::OnNodeDoubleClicked(UEdGraphNode* Node)
{
	if (UEdGraphNode_PluginReference* PlugingReferenceNode = Cast<UEdGraphNode_PluginReference>(Node))
	{
		if (PlugingReferenceNode->GetPlugin().IsValid())
		{
			TSet<UObject*> Nodes;
			Nodes.Add(Node);
			ReCenterGraphOnNodes(Nodes);
		}
	}
}

void SPluginReferenceViewer::RegisterActions()
{
	PluginReferenceViewerActions = MakeShareable(new FUICommandList);
	FPluginReferenceViewerCommands::Register();

	PluginReferenceViewerActions->MapAction(
		FPluginReferenceViewerCommands::Get().CompactMode,
		FExecuteAction::CreateSP(this, &SPluginReferenceViewer::OnCompactModeChanged),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SPluginReferenceViewer::IsCompactModeChecked));

	PluginReferenceViewerActions->MapAction(
		FPluginReferenceViewerCommands::Get().ShowDuplicates,
		FExecuteAction::CreateSP(this, &SPluginReferenceViewer::OnShowDuplicatesChanged),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SPluginReferenceViewer::IsShowDuplicatesChecked));

	PluginReferenceViewerActions->MapAction(
		FPluginReferenceViewerCommands::Get().ShowEnginePlugins,
		FExecuteAction::CreateSP(this, &SPluginReferenceViewer::OnShowEnginePluginsChanged),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SPluginReferenceViewer::IsShowEnginePluginsChecked));

	PluginReferenceViewerActions->MapAction(
		FPluginReferenceViewerCommands::Get().ShowOptionalPlugins,
		FExecuteAction::CreateSP(this, &SPluginReferenceViewer::OnShowOptionalPluginsChanged),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SPluginReferenceViewer::IsShowOptionalPluginsChecked));

	PluginReferenceViewerActions->MapAction(
		FPluginReferenceViewerCommands::Get().OpenPluginProperties,
		FExecuteAction::CreateSP(this, &SPluginReferenceViewer::OnOpenPluginProperties),
		FCanExecuteAction::CreateSP(this, &SPluginReferenceViewer::HasAtLeastOneRealNodeSelected));

	PluginReferenceViewerActions->MapAction(
		FPluginReferenceViewerCommands::Get().OpenAssetDiskSizeViewer,
		FExecuteAction::CreateSP(this, &SPluginReferenceViewer::OnOpenAssetDiskSizeViewer),
		FCanExecuteAction::CreateSP(this, &SPluginReferenceViewer::HasAtLeastOneRealNodeSelected));
	
	PluginReferenceViewerActions->MapAction(
		FPluginReferenceViewerCommands::Get().ShowInContentBrowser,
		FExecuteAction::CreateSP(this, &SPluginReferenceViewer::OnShowPluginInContentBrowser),
		FCanExecuteAction::CreateSP(this, &SPluginReferenceViewer::HasAtLeastOneRealNodeSelected));

	PluginReferenceViewerActions->MapAction(
		FPluginReferenceViewerCommands::Get().ZoomToFit,
		FExecuteAction::CreateSP(this, &SPluginReferenceViewer::ZoomToFit),
		FCanExecuteAction());

	PluginReferenceViewerActions->MapAction(
		FPluginReferenceViewerCommands::Get().ReCenterGraph,
		FExecuteAction::CreateSP(this, &SPluginReferenceViewer::ReCenterGraph),
		FCanExecuteAction());
}

bool SPluginReferenceViewer::IsCompactModeChecked() const
{
	return Settings.bIsCompactMode;
}

void SPluginReferenceViewer::OnCompactModeChanged()
{
	Settings.bIsCompactMode = !Settings.bIsCompactMode;
	GraphObj->RefilterGraph();
}

void SPluginReferenceViewer::OnShowDuplicatesChanged()
{
	Settings.bShowDuplicates = !Settings.bShowDuplicates;
	GraphObj->RefilterGraph();
}

bool SPluginReferenceViewer::IsShowDuplicatesChecked() const
{
	return Settings.bShowDuplicates;
}

bool SPluginReferenceViewer::IsShowEnginePluginsChecked() const
{
	return Settings.bShowEnginePlugins;
}

void SPluginReferenceViewer::OnShowEnginePluginsChanged()
{
	Settings.bShowEnginePlugins = !Settings.bShowEnginePlugins;
	GraphObj->RebuildGraph();
}

bool SPluginReferenceViewer::IsShowOptionalPluginsChecked() const
{
	return Settings.bShowOptionalPlugins;
}

void SPluginReferenceViewer::OnShowOptionalPluginsChanged()
{
	Settings.bShowOptionalPlugins = !Settings.bShowOptionalPlugins;
	GraphObj->RebuildGraph();
}

int32 SPluginReferenceViewer::GetSearchReferencerDepthCount() const
{
	return Settings.MaxSearchReferencersDepth;
}

int32 SPluginReferenceViewer::GetSearchDependencyDepthCount() const
{
	return Settings.MaxSearchDependencyDepth;
}

TSharedRef<SDockTab> SPluginReferenceViewer::SpawnTab_ReferenceGraph(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
	[
		SNew(SVerticalBox)

		// Path and history
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4, 0)
				[
					MakeToolBar()
				]

				// Path
				+SHorizontalBox::Slot()
				.Padding(0, 0, 4, 0)
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
					[
						SNew(SEditableTextBox)
						.Text(this, &SPluginReferenceViewer::GetAddressBarText)
						.OnTextCommitted(this, &SPluginReferenceViewer::OnAddressBarTextCommitted)
						.OnTextChanged(this, &SPluginReferenceViewer::OnAddressBarTextChanged)
						.SelectAllTextWhenFocused(true)
						.SelectAllTextOnCommit(true)
						.Style(FAppStyle::Get(), "ReferenceViewer.PathText")
					]
				]
			]
		]

		// Graph
		+SVerticalBox::Slot()
		.FillHeight(0.90f)
		.HAlign(HAlign_Fill)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				GraphEditorPtr.ToSharedRef()
			]

			+SOverlay::Slot()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Fill)
			.Padding(8)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(2.f)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("SearchDepthReferencersLabelText", "Search Referencers Depth"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(2.f)
							[
								SNew(SBox)
								.WidthOverride(100)
								[
									SAssignNew(ReferencerCountBox, SSpinBox<int32>)
									.Value(this, &SPluginReferenceViewer::GetSearchReferencerDepthCount)
									.OnValueChanged_Lambda([this] (int32 NewValue)
										{	
											if (NewValue != Settings.MaxSearchReferencersDepth)
											{
												Settings.MaxSearchReferencersDepth = NewValue;
												bNeedsGraphRebuild = true;
											}
										}
									)
									.OnValueCommitted_Lambda([this] (int32 NewValue, ETextCommit::Type CommitType) 
										{ 
											FSlateApplication::Get().SetKeyboardFocus(GraphEditorPtr, EFocusCause::SetDirectly); 

											if (NewValue != Settings.MaxSearchReferencersDepth || bNeedsGraphRebuild)
											{
												Settings.MaxSearchReferencersDepth = NewValue;
												bNeedsGraphRebuild = false;
												RebuildGraph();
											}
										}
									)
									.MinValue(0)
									.MaxValue(50)
									.MaxSliderValue(10)
								]
							]
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(2.f)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("SearchDepthDependenciesLabelText", "Search Dependencies Depth"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(2.f)
							[
								SNew(SBox)
								.WidthOverride(100)
								[
									SAssignNew(DependencyCountBox, SSpinBox<int32>)
									.Value(this, &SPluginReferenceViewer::GetSearchDependencyDepthCount)
									.OnValueChanged_Lambda([this] (int32 NewValue)
										{	
											if (NewValue != Settings.MaxSearchDependencyDepth)
											{
												Settings.MaxSearchDependencyDepth = NewValue;
												bNeedsGraphRebuild = true;
											}
										}
									)
									.OnValueCommitted_Lambda([this] (int32 NewValue, ETextCommit::Type CommitType) 
										{ 
											FSlateApplication::Get().SetKeyboardFocus(GraphEditorPtr, EFocusCause::SetDirectly); 

											if (NewValue != Settings.MaxSearchDependencyDepth || bNeedsGraphRebuild)
											{
												Settings.MaxSearchDependencyDepth = NewValue;
												bNeedsGraphRebuild = false;
												RebuildGraph();
											}
										}
									)
									.MinValue(0)
									.MaxValue(50)
									.MaxSliderValue(10)
								]
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(2.f)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("ShowAdvancedInfoLabelText", "Show Advanced Info"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(2.f)
							[
								SNew(SCheckBox)
									.IsChecked_Lambda([this]()
										{
											return bShowAdvancedInfo ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
										})
									.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
										{
											if (InCheckBoxState == ECheckBoxState::Checked && GraphObj != nullptr)
											{
												// need to find a way to do a loading thing here
												GraphObj->LoadAdvancedPluginInfo();
												bShowAdvancedInfo = true;
												TabManager->TryInvokeTab(PluginReferenceViewerTabID::PluginStats);
											}
											else
											{
												TSharedPtr<SDockTab> TabPtr = TabManager->FindExistingLiveTab(PluginReferenceViewerTabID::PluginStats);
												if (TabPtr.IsValid())
												{
													TabPtr->RequestCloseTab();
												}
												// we don't want to actually change our state if we weren't able to load advanced info
												bShowAdvancedInfo = false;
											}
										})
							]
						]
					]

				]
			]
		]
	];

	return SpawnedTab;
}

TSharedRef<SDockTab> SPluginReferenceViewer::SpawnTab_PluginStats(const FSpawnTabArgs& Args) const
{
	const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.ShouldAutosize(false)
		.TabRole(ETabRole::PanelTab)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &SPluginReferenceViewer::GetPluginStatsText)
			]
		];

	return DockTab;
}

void SPluginReferenceViewer::OnTabSpawned(const FName& TabIdentifier, const TSharedRef<SDockTab>& SpawnedTab)
{
	TWeakPtr<SDockTab>* const ExistingTab = SpawnedTabs.Find(TabIdentifier);
	if (ExistingTab)
	{
		if (ExistingTab->IsValid())
		{
			*ExistingTab = SpawnedTab;
		}
	}
	else
	{
		SpawnedTabs.Add(TabIdentifier, SpawnedTab);
	}
}

#undef LOCTEXT_NAMESPACE
