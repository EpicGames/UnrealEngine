// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "EdGraph_PluginReferenceViewer.h"
#include "GraphEditor.h"
#include "HistoryManager.h"

#include "Framework/Docking/TabManager.h"

//////////////////////////////////////////////////////////////////////////
// SPluginReferenceViewer

class IPlugin;
class FUICommandInfo;
class FUICommandList;
class SDockTab;
class UEdGraphNode;
class UEdGraph_PluginReferenceViewer;

class SPluginReferenceViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPluginReferenceViewer)
	{}

	SLATE_ARGUMENT(TSharedPtr<SDockTab>, ParentTab)

	SLATE_END_ARGS()

	~SPluginReferenceViewer();

public:
	void Construct(const FArguments& InArgs);

	void SetGraphRootIdentifiers(const TArray<FPluginIdentifier>& GraphRootIdentifiers);

	int32 GetSearchReferencerDepthCount() const;
	int32 GetSearchDependencyDepthCount() const;
	bool IsCompactModeChecked() const;
	bool IsShowDuplicatesChecked() const;
	bool IsShowEnginePluginsChecked() const;
	bool IsShowOptionalPluginsChecked() const;

private:
	void OnOpenPluginProperties();
	void OnOpenAssetDiskSizeViewer();
	void OnShowPluginInContentBrowser();
	bool HasAtLeastOneRealNodeSelected();

	void OpenPluginProperties(const FString& PluginName);

	TSharedRef<SWidget> MakeToolBar();
	TSharedRef<SWidget> GetShowMenuContent();

	void RebuildGraph();
	void ZoomToFit();
	void ReCenterGraph();
	void ReCenterGraphOnNodes(const TSet<UObject*>& Nodes);
	void RegisterActions();

	/** Called when a node is double clicked */
	void OnNodeDoubleClicked(UEdGraphNode* Node);

	/** True if the user may use the history back button */
	bool IsBackEnabled() const;

	/** True if the user may use the history forward button */
	bool IsForwardEnabled() const;

	/** Handler for clicking the history back button */
	void BackClicked();

	/** Handler for clicking the history forward button */
	void ForwardClicked();

	/** Refresh the current view */
	void RefreshClicked();

	/** Handler for when the graph panel tells us to go back in history (like using the mouse thumb button) */
	void GraphNavigateHistoryBack();

	/** Handler for when the graph panel tells us to go forward in history (like using the mouse thumb button) */
	void GraphNavigateHistoryForward();

	/** Gets the tool tip text for the history back button */
	FText GetHistoryBackTooltip() const;

	/** Gets the tool tip text for the history forward button */
	FText GetHistoryForwardTooltip() const;

	/** Gets text for advanced info plugin stats. Formatted output of plugin content size, reference counts, etc. */
	FText GetPluginStatsText() const;

	FText GetAddressBarText() const;
	void OnAddressBarTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);
	void OnAddressBarTextChanged(const FText& NewText);

	void OnCompactModeChanged();
	void OnShowDuplicatesChanged();
	void OnShowEnginePluginsChanged();
	void OnShowOptionalPluginsChanged();
	void OnApplyHistoryData(const FPluginReferenceViewerHistoryData& History);
	void OnUpdateHistoryData(FPluginReferenceViewerHistoryData& HistoryData) const;

	struct FPluginReferenceViewerSettings
	{
		int32 MaxSearchReferencersDepth = 1;
		int32 MaxSearchDependencyDepth = 1;
		bool bIsCompactMode = false;
		bool bShowDuplicates = true;
		bool bShowEnginePlugins = true;
		bool bShowOptionalPlugins = true;
	};

	void OnTabSpawned(const FName& TabIdentifier, const TSharedRef<SDockTab>& SpawnedTab);
	TSharedRef<SDockTab> SpawnTab_ReferenceGraph(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_PluginStats(const FSpawnTabArgs& Args) const;

private:
	/** The manager that keeps track of history data for this browser */
	FPluginReferenceViewerHistoryManager HistoryManager;

	TSharedPtr<SGraphEditor> GraphEditorPtr;
	TSharedPtr<FUICommandList> PluginReferenceViewerActions;
	TSharedPtr<SWidget> ReferencerCountBox;
	TSharedPtr<SWidget> DependencyCountBox;

	UEdGraph_PluginReferenceViewer* GraphObj;
	FPluginReferenceViewerSettings Settings;

	/** The temporary copy of the path text when it is actively being edited. */
	FText TemporaryPathBeingEdited;

	TSharedPtr<FTabManager> TabManager;
	TMap<FName, TWeakPtr<SDockTab>> SpawnedTabs;

	TSharedPtr<FUICommandInfo> ShowEnginePlugins;

	/** Used to delay graph rebuilding during spinbox slider interaction */
	bool bNeedsGraphRebuild;

	/** Enables loading of plugin stats, such as reference counts, plugin content size, and dependant content size. */
	bool bShowAdvancedInfo;
};