// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"

#define UE_API PLUGINREFERENCEVIEWER_API

class FPluginReferenceViewerGraphPanelNodeFactory;

class FPluginReferenceViewerModule : public IModuleInterface
{
public:
	static inline FPluginReferenceViewerModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FPluginReferenceViewerModule>("PluginReferenceViewer");
	}

	// IModuleInterface interface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	// End of IModuleInterface interface

	UE_API void OpenPluginReferenceViewerUI(const TSharedRef<class IPlugin>& Plugin);

private:
	UE_API void OnLaunchReferenceViewerFromPluginBrowser(TSharedPtr<IPlugin> Plugin);

	static UE_API const FName PluginReferenceViewerTabName;

	UE_API TSharedRef<SDockTab> SpawnPluginReferenceViewerTab(const FSpawnTabArgs& Args);

	TSharedPtr<FPluginReferenceViewerGraphPanelNodeFactory> PluginReferenceViewerGraphPanelNodeFactory;
};

#undef UE_API
