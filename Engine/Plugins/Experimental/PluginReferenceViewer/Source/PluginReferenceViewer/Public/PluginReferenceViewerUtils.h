// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API PLUGINREFERENCEVIEWER_API

class IPlugin;
struct FAssetIdentifier;

class FPluginReferenceViewerUtils
{
public:
	/* Helper that returns array of Asset dependencies for a given plugin*/
	static UE_API TMap<FAssetIdentifier, TArray<FAssetIdentifier>> GetAssetDependencyMap(const TSharedRef<IPlugin>& InPlugin);

	/* Helper that returns array of Asset dependencies for a given plugin*/
	static UE_API TArray<FAssetIdentifier> GetAssetDependencies(const TSharedRef<IPlugin>& InPlugin);

	/* Helper that will split plugin dependencies by their owning plugin*/
	static UE_API TMap<FString, TArray<FAssetIdentifier>> SplitByPlugins(const TSharedRef<IPlugin>& InOwningPlugin, const TArray<FAssetIdentifier>& InPluginDependencies);

	/* Helper that will split an array of assets by their reference types*/
	static UE_API void SplitByReferenceType(const TArray<FAssetIdentifier>& InAssetIdentifiers, TArray<FAssetIdentifier>& OutAssetReferences, TArray<FAssetIdentifier>& OutScriptReferences, TArray<FAssetIdentifier>& OutNameReferences);

	/* Exports the number of references from each plugin to their plugin dependencies. References include; assets, scripts and named references*/
	static UE_API void ExportPlugins(const TArray<FString>& InPlugins, const FString& InFilename);

	/* Exports to a .csv file the list of asset references that exist between the plugin and one of it's plugin dependencies by asset */
	static UE_API void ExportAssetReference(const FString& InPlugin, const FString& InReference, const FString& InFilename);

	/* Exports to a .csv file the list of asset references that exist between the plugin and one of it's plugin dependencies */
	static UE_API void ExportReference(const FString& InPlugin, const FString& InReference, const FString& InFilename);

	/* Returns the list of plugins where the supplied gameplay tag is declated */
	static UE_API TArray<TSharedRef<IPlugin>> FindGameplayTagSourcePlugins(FName TagName);

	/*
	 * Traces the path from one plugin to the ending plugin if one exists.
	 * Returns true if a path exists.
	 */
	static UE_API bool TracePluginChain(const FString& StartingPlugin, const FString& EndingPlugin, FString& OutPathToEndPlugin);

private:
	FPluginReferenceViewerUtils() = delete;
	FPluginReferenceViewerUtils(const FPluginReferenceViewerUtils&) = delete;
};

#undef UE_API
